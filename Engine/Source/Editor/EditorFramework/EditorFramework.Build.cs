// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class EditorFramework : ModuleRules
{
	public EditorFramework(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"ToolMenus"
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"EditorSubsystem",
				"InteractiveToolsFramework",
				"TypedElementFramework",
				"TypedElementRuntime",
			}
		);
	}
}
