// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorFrameworkModule.h"

#include "Modules/ModuleManager.h"

void FEditorFrameworkModule::StartupModule()
{
}

void FEditorFrameworkModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FEditorFrameworkModule, EditorFramework);
