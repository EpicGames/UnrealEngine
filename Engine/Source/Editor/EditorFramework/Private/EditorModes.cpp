// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorModes.h"

#include "Editor.h"
#include "HAL/Platform.h"
#include "UObject/UnrealNames.h"


DEFINE_LOG_CATEGORY(LogEditorModes);

// Builtin editor mode constants
namespace FBuiltinEditorModes
{
	const FEditorModeID EM_None = NAME_None;
	const FEditorModeID EM_Default(TEXT("EM_Default"));
	const FEditorModeID EM_Placement(TEXT("PLACEMENT"));
	const FEditorModeID EM_MeshPaint(TEXT("EM_MeshPaint"));
	const FEditorModeID EM_Landscape(TEXT("EM_Landscape"));
	const FEditorModeID EM_Foliage(TEXT("EM_Foliage"));
	const FEditorModeID EM_Level(TEXT("EM_Level"));
	const FEditorModeID EM_StreamingLevel(TEXT("EM_StreamingLevel"));
	const FEditorModeID EM_Physics(TEXT("EM_Physics"));
	const FEditorModeID EM_ActorPicker(TEXT("EM_ActorPicker"));
	const FEditorModeID EM_SceneDepthPicker(TEXT("EM_SceneDepthPicker"));
}
