// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorViewportLayout.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"
#include "Framework/Docking/LayoutService.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SCanvas.h"
#include "Styling/AppStyle.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "LevelEditor.h"
#include "Widgets/Docking/SDockTab.h"
#include "ViewportTabContent.h"
#if 0

static const FName LevelEditorModName("LevelEditor");

namespace ViewportLayoutDefs
{
	/** How many seconds to interpolate from restored to maximized state */
	static const float MaximizeTransitionTime = 0.15f;

	/** How many seconds to interpolate from maximized to restored state */
	static const float RestoreTransitionTime = 0.2f;

	/** Default maximized state for new layouts - will only be applied when no config data is restoring state */
	static const bool bDefaultShouldBeMaximized = true;

	/** Default immersive state for new layouts - will only be applied when no config data is restoring state */
	static const bool bDefaultShouldBeImmersive = false;
}

// SViewportsOverlay ////////////////////////////////////////////////

/**
 * Overlay wrapper class so that we can cache the size of the widget
 * It will also store the LevelViewportLayout data because that data can't be stored
 * per app; it must be stored per viewport overlay in case the app that made it closes.
 */
class SViewportsOverlay : public SCompoundWidget
{

public:

	SLATE_BEGIN_ARGS( SViewportsOverlay ){}
		SLATE_DEFAULT_SLOT( FArguments, Content )
		SLATE_ARGUMENT( TSharedPtr<FViewportTabContent>, LevelViewportTab )
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs );

	/** Default constructor */
	SViewportsOverlay()
		: CachedSize( FVector2D::ZeroVector )
	{}

	/** Overridden from SWidget */
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	/** Wraps SOverlay::AddSlot() */
	SOverlay::FOverlaySlot& AddSlot();

	/** Wraps SOverlay::RemoveSlot() */
	void RemoveSlot();

	/**
	 * Returns the cached size of this viewport overlay
	 *
	 * @return	The size that was cached
	 */
	const FVector2D& GetCachedSize() const;

	/** Gets the Level Viewport Tab that created this overlay */
	TSharedPtr<FViewportTabContent> GetLevelViewportTab() const;

private:
	
	/** Reference to the owning level viewport tab */
	TSharedPtr<FViewportTabContent> LevelViewportTab;

	/** The overlay widget we're containing */
	TSharedPtr< SOverlay > OverlayWidget;

	/** Cache our size, so that we can use this when animating a viewport maximize/restore */
	FVector2D CachedSize;
};


void SViewportsOverlay::Construct( const FArguments& InArgs )
{
	const TSharedRef<SWidget>& ContentWidget = InArgs._Content.Widget;
	LevelViewportTab = InArgs._LevelViewportTab;

	ChildSlot
		[
			SAssignNew( OverlayWidget, SOverlay )
			+ SOverlay::Slot()
			[
				ContentWidget
			]
		];
}

void SViewportsOverlay::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	CachedSize = AllottedGeometry.Size;
}

SOverlay::FOverlaySlot& SViewportsOverlay::AddSlot()
{
	return OverlayWidget->AddSlot();
}

void SViewportsOverlay::RemoveSlot()
{
	return OverlayWidget->RemoveSlot();
}

const FVector2D& SViewportsOverlay::GetCachedSize() const
{
	return CachedSize;
}

TSharedPtr<FViewportTabContent> SViewportsOverlay::GetLevelViewportTab() const
{
	return LevelViewportTab;
}

#endif