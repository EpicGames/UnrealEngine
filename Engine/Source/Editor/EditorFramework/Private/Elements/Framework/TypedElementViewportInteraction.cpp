// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementViewportInteraction.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "EditorModeManager.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Elements/Framework/TypedElementId.h"
#include "Elements/Framework/TypedElementLimits.h"
#include "Elements/Framework/TypedElementList.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementUtil.h"
#include "Math/Box.h"
#include "Math/BoxSphereBounds.h"
#include "Math/Matrix.h"
#include "Math/Quat.h"
#include "Math/RotationMatrix.h"
#include "Math/Rotator.h"
#include "Math/ScaleMatrix.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Toolkits/IToolkitHost.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementViewportInteraction)

struct FInputDeviceState;

bool FTypedElementViewportInteractionCustomization::GetGizmoPivotLocation(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, FVector& OutPivotLocation)
{
	if (const IToolkitHost* ToolkitHostPtr = GetToolkitHost())
	{
		OutPivotLocation = ToolkitHostPtr->GetEditorModeManager().PivotLocation;
		return true;
	}

	FTransform ElementWorldTransform;
	if (InElementWorldHandle.GetWorldTransform(ElementWorldTransform))
	{
		OutPivotLocation = ElementWorldTransform.GetTranslation();
		return true;
	}

	return false;
}

void FTypedElementViewportInteractionCustomization::PreGizmoManipulationStarted(TArrayView<const FTypedElementHandle> InElementHandles, const UE::Widget::EWidgetMode InWidgetMode)
{
}

void FTypedElementViewportInteractionCustomization::GizmoManipulationStarted(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode)
{
	InElementWorldHandle.NotifyMovementStarted();
}

void FTypedElementViewportInteractionCustomization::GizmoManipulationDeltaUpdate(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, const EAxisList::Type InDragAxis, const FInputDeviceState& InInputState, const FTransform& InDeltaTransform, const FVector& InPivotLocation)
{
	FTransform ElementWorldTransform;
	if (InElementWorldHandle.GetWorldTransform(ElementWorldTransform))
	{
		// Apply delta rotation around the pivot location
		{
			const FQuat DeltaRotation = InDeltaTransform.GetRotation();
			if (!DeltaRotation.Rotator().IsZero())
			{
				const FQuat ActorQ =  ElementWorldTransform.GetRotation();
				const FQuat ResultQ = DeltaRotation * ActorQ;
				ElementWorldTransform.SetRotation(ResultQ.GetNormalized());

				FVector ElementLocation = ElementWorldTransform.GetTranslation();
				ElementLocation -= InPivotLocation;
				ElementLocation = FRotationMatrix::Make(DeltaRotation).TransformPosition(ElementLocation);
				ElementLocation += InPivotLocation;
				ElementWorldTransform.SetTranslation(ElementLocation);
			}
		}

		// Apply delta translation
		{
			const FVector DeltaTranslation = InDeltaTransform.GetTranslation();
			ElementWorldTransform.SetTranslation(ElementWorldTransform.GetTranslation() + DeltaTranslation);
		}

		// Apply delta scaling around the pivot location
		{
			const FVector DeltaScale3D = InDeltaTransform.GetScale3D();
			if (!DeltaScale3D.IsNearlyZero(0.000001f))
			{
				ElementWorldTransform.SetScale3D(ElementWorldTransform.GetScale3D() + DeltaScale3D);

				FVector ElementLocation = ElementWorldTransform.GetTranslation();
				ElementLocation -= InPivotLocation;
				ElementLocation += FScaleMatrix::Make(DeltaScale3D).TransformPosition(ElementLocation);
				ElementLocation += InPivotLocation;
				ElementWorldTransform.SetTranslation(ElementLocation);
			}
		}

		InElementWorldHandle.SetWorldTransform(ElementWorldTransform);
		InElementWorldHandle.NotifyMovementOngoing();
	}
}

void FTypedElementViewportInteractionCustomization::GizmoManipulationStopped(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, const ETypedElementViewportInteractionGizmoManipulationType InManipulationType)
{
	InElementWorldHandle.NotifyMovementEnded();
}

void FTypedElementViewportInteractionCustomization::PostGizmoManipulationStopped(TArrayView<const FTypedElementHandle> InElementHandles, const UE::Widget::EWidgetMode InWidgetMode)
{
}

void FTypedElementViewportInteractionCustomization::MirrorElement(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const FVector& InMirrorScale, const FVector& InPivotLocation)
{
	FTransform ElementWorldTransform;
	if (InElementWorldHandle.GetWorldTransform(ElementWorldTransform))
	{
		InElementWorldHandle.NotifyMovementStarted();

		// Apply mirrored rotation
		{
			// Revert the handedness of the rotation, but make up for it in the scaling
			// Arbitrarily choose the X axis to remain fixed
			const FMatrix TempRot = FRotationMatrix::Make(ElementWorldTransform.GetRotation());
			const FMatrix NewRot(
				-TempRot.GetScaledAxis(EAxis::X) * InMirrorScale, 
				TempRot.GetScaledAxis(EAxis::Y) * InMirrorScale, 
				TempRot.GetScaledAxis(EAxis::Z) * InMirrorScale, 
				FVector::ZeroVector
				);
			ElementWorldTransform.SetRotation(NewRot.ToQuat());
		}

		// Apply mirrored location around the pivot location
		{
			FVector Loc = ElementWorldTransform.GetTranslation();
			Loc -= InPivotLocation;
			Loc *= InMirrorScale;
			Loc += InPivotLocation;
			ElementWorldTransform.SetTranslation(Loc);
		}

		InElementWorldHandle.SetWorldTransform(ElementWorldTransform);

		// Apply mirrored relative scale
		{
			FTransform ElementRelativeTransform;
			if (InElementWorldHandle.GetRelativeTransform(ElementRelativeTransform))
			{
				FVector Scale3D = ElementRelativeTransform.GetScale3D();
				Scale3D.X = -Scale3D.X;
				ElementRelativeTransform.SetScale3D(Scale3D);

				InElementWorldHandle.SetRelativeTransform(ElementRelativeTransform);
			}
		}

		InElementWorldHandle.NotifyMovementEnded();
	}
}

bool FTypedElementViewportInteractionCustomization::GetFocusBounds(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, FBoxSphereBounds& OutBounds)
{
	return InElementWorldHandle.GetBounds(OutBounds);
}

bool FTypedElementViewportInteractionCustomization::GetScaleForGizmo(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, FVector& OutScale)
{
	FTransform Transform;
	if (InElementWorldHandle.GetRelativeTransform(Transform))
	{
		OutScale = Transform.GetScale3D();
		return true;
	}

	return false;
}

void UTypedElementViewportInteraction::BeginGizmoManipulation(FTypedElementListConstRef InElementsToMove, const UE::Widget::EWidgetMode InWidgetMode)
{
	{
		TMap<FTypedHandleTypeId, TArray<FTypedElementHandle>> ElementsToMoveByType;
		TypedElementUtil::BatchElementsByType(InElementsToMove, ElementsToMoveByType);

		for (const auto& ElementsByTypePair : ElementsToMoveByType)
		{
			FTypedElementViewportInteractionCustomization* ViewportInteractionCustomization = GetInterfaceCustomizationByTypeId(ElementsByTypePair.Key);
			check(ViewportInteractionCustomization);
			ViewportInteractionCustomization->PreGizmoManipulationStarted(ElementsByTypePair.Value, InWidgetMode);
		}
	}

	InElementsToMove->ForEachElementHandle([this, InWidgetMode](const FTypedElementHandle& InElementToMove)
	{
		FTypedElementViewportInteractionElement ViewportInteractionElement = ResolveViewportInteractionElement(InElementToMove);
		if (ViewportInteractionElement)
		{
			ViewportInteractionElement.GizmoManipulationStarted(InWidgetMode);
		}
		return true;
	});
}

void UTypedElementViewportInteraction::UpdateGizmoManipulation(FTypedElementListConstRef InElementsToMove, const UE::Widget::EWidgetMode InWidgetMode, const EAxisList::Type InDragAxis, const FInputDeviceState& InInputState, const FTransform& InDeltaTransform)
{
	InElementsToMove->ForEachElementHandle([this, InWidgetMode, InDragAxis, &InInputState, &InDeltaTransform](const FTypedElementHandle& InElementToMove)
	{
		FTypedElementViewportInteractionElement ViewportInteractionElement = ResolveViewportInteractionElement(InElementToMove);
		if (ViewportInteractionElement)
		{
			FVector PivotLocation = FVector::ZeroVector;
			ViewportInteractionElement.GetGizmoPivotLocation(InWidgetMode, PivotLocation);
			ViewportInteractionElement.GizmoManipulationDeltaUpdate(InWidgetMode, InDragAxis, InInputState, InDeltaTransform, PivotLocation);
		}
		return true;
	});
}

void UTypedElementViewportInteraction::EndGizmoManipulation(FTypedElementListConstRef InElementsToMove, const UE::Widget::EWidgetMode InWidgetMode, const ETypedElementViewportInteractionGizmoManipulationType InManipulationType)
{
	InElementsToMove->ForEachElementHandle([this, InWidgetMode, InManipulationType](const FTypedElementHandle& InElementToMove)
	{
		FTypedElementViewportInteractionElement ViewportInteractionElement = ResolveViewportInteractionElement(InElementToMove);
		if (ViewportInteractionElement)
		{
			ViewportInteractionElement.GizmoManipulationStopped(InWidgetMode, InManipulationType);
		}
		return true;
	});
	
	{
		TMap<FTypedHandleTypeId, TArray<FTypedElementHandle>> ElementsToMoveByType;
		TypedElementUtil::BatchElementsByType(InElementsToMove, ElementsToMoveByType);

		for (const auto& ElementsByTypePair : ElementsToMoveByType)
		{
			FTypedElementViewportInteractionCustomization* ViewportInteractionCustomization = GetInterfaceCustomizationByTypeId(ElementsByTypePair.Key);
			check(ViewportInteractionCustomization);
			ViewportInteractionCustomization->PostGizmoManipulationStopped(ElementsByTypePair.Value, InWidgetMode);
		}
	}
}

void UTypedElementViewportInteraction::ApplyDeltaToElement(const FTypedElementHandle& InElementHandle, const UE::Widget::EWidgetMode InWidgetMode, const EAxisList::Type InDragAxis, const FInputDeviceState& InInputState, const FTransform& InDeltaTransform)
{
	FTypedElementViewportInteractionElement ViewportInteractionElement = ResolveViewportInteractionElement(InElementHandle);
	if (ViewportInteractionElement)
	{
		FVector PivotLocation = FVector::ZeroVector;
		ViewportInteractionElement.GetGizmoPivotLocation(InWidgetMode, PivotLocation);
		ViewportInteractionElement.GizmoManipulationDeltaUpdate(InWidgetMode, InDragAxis, InInputState, InDeltaTransform, PivotLocation);
	}
}

void UTypedElementViewportInteraction::MirrorElement(const FTypedElementHandle& InElementHandle, const FVector& InMirrorScale)
{
	FTypedElementViewportInteractionElement ViewportInteractionElement = ResolveViewportInteractionElement(InElementHandle);
	if (ViewportInteractionElement)
	{
		FVector PivotLocation = FVector::ZeroVector;
		ViewportInteractionElement.GetGizmoPivotLocation(UE::Widget::WM_None, PivotLocation);
		ViewportInteractionElement.MirrorElement(InMirrorScale, PivotLocation);
	}
}

bool UTypedElementViewportInteraction::GetFocusBounds(FTypedElementListConstRef InElements, FBoxSphereBounds& OutBounds)
{
	bool bAnyHaveBounds = false;
	FBox TotalBounds(ForceInit);
	InElements->ForEachElementHandle([this, &TotalBounds, &bAnyHaveBounds](const FTypedElementHandle& InElementToMove)
	{
		FTypedElementViewportInteractionElement ViewportInteractionElement = ResolveViewportInteractionElement(InElementToMove);
		if (ViewportInteractionElement)
		{
			FBoxSphereBounds ElementBounds;
			if (ViewportInteractionElement.GetFocusBounds(ElementBounds))
			{
				bAnyHaveBounds = true;
				TotalBounds += ElementBounds.GetBox();
			}
		}
		return true;
	});

	OutBounds = TotalBounds;
	return bAnyHaveBounds;
}

bool UTypedElementViewportInteraction::GetScaleForGizmo(FTypedElementListConstRef InElements, FVector& OutScale)
{
	bool bHasAnyScale = false;
	FVector CombinedScale(ForceInitToZero);
	InElements->ForEachElementHandle([this, &CombinedScale, &bHasAnyScale](const FTypedElementHandle& InElementToMove)
	{
		FTypedElementViewportInteractionElement ViewportInteractionElement = ResolveViewportInteractionElement(InElementToMove);
		if (ViewportInteractionElement)
		{
			FVector ElementScale;
			if (ViewportInteractionElement.GetScaleForGizmo(ElementScale))
			{
				if (!bHasAnyScale)
				{
					CombinedScale = ElementScale;
				}
				else
				{
					CombinedScale = FVector::Max(CombinedScale, ElementScale);
				}

				bHasAnyScale = true;
			}
		}
		return true;
	});

	OutScale = CombinedScale;
	return bHasAnyScale;
}

FTypedElementViewportInteractionElement UTypedElementViewportInteraction::ResolveViewportInteractionElement(const FTypedElementHandle& InElementHandle) const
{
	return InElementHandle
		? FTypedElementViewportInteractionElement(UTypedElementRegistry::GetInstance()->GetElement<ITypedElementWorldInterface>(InElementHandle), GetInterfaceCustomizationByTypeId(InElementHandle.GetId().GetTypeId()))
		: FTypedElementViewportInteractionElement();
}
