// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/AssetFactoryInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetFactoryInterface)
