// Copyright Epic Games, Inc. All Rights Reserved.


#include "Styling/DepthBarStyle.h"

#include "Brushes/SlateNoResource.h"

const FName FDepthBarStyle::TypeName("FDepthBarStyle");

FDepthBarStyle::FDepthBarStyle()
	: BackgroundBrush(FSlateNoResource())
	, TrackBrush(FSlateNoResource())
	, SliceNormalBrush(FSlateNoResource())
	, SliceHoveredBrush(FSlateNoResource())
	, SliceTopBrush(FSlateNoResource())
	, SliceBottomBrush(FSlateNoResource())
	, SliceBottomHoveredBrush(FSlateNoResource())
	, FarPlaneButtonBrush(FSlateNoResource())
	, NearPlaneButtonBrush(FSlateNoResource())
{
}

FDepthBarStyle::FDepthBarStyle(const FDepthBarStyle&) = default;

FDepthBarStyle::~FDepthBarStyle() = default;

const FDepthBarStyle& FDepthBarStyle::GetDefault()
{
	static FDepthBarStyle Default;
	return Default;
}

void FDepthBarStyle::GetResources(TArray<const FSlateBrush*>& OutBrushes) const
{
	OutBrushes.Add(&BackgroundBrush);
	OutBrushes.Add(&TrackBrush);
	OutBrushes.Add(&SliceNormalBrush);
	OutBrushes.Add(&SliceHoveredBrush);
	OutBrushes.Add(&SliceTopBrush);
	OutBrushes.Add(&SliceTopHoveredBrush);
	OutBrushes.Add(&SliceBottomBrush);
	OutBrushes.Add(&SliceBottomHoveredBrush);
	OutBrushes.Add(&FarPlaneButtonBrush);
	OutBrushes.Add(&NearPlaneButtonBrush);
}
