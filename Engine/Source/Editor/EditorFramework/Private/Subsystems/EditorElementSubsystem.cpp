// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/EditorElementSubsystem.h"

#include "Elements/Framework/TypedElementHandle.h"
#include "Elements/Framework/TypedElementList.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorElementSubsystem)

bool UEditorElementSubsystem::SetElementTransform(FTypedElementHandle InElementHandle, const FTransform& InWorldTransform)
{
	if (TTypedElement<ITypedElementWorldInterface> WorldInterfaceElement = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementWorldInterface>(InElementHandle))
	{
		if (UWorld* ElementWorld = WorldInterfaceElement.GetOwnerWorld())
		{
			ETypedElementWorldType WorldType = ElementWorld->IsGameWorld() ? ETypedElementWorldType::Game : ETypedElementWorldType::Editor;
			if (WorldInterfaceElement.CanMoveElement(WorldType))
			{
				WorldInterfaceElement.NotifyMovementStarted();
				WorldInterfaceElement.SetWorldTransform(InWorldTransform);
				WorldInterfaceElement.NotifyMovementEnded();

				return true;
			}
		}
	}

	return false;
}

FTypedElementListRef UEditorElementSubsystem::GetEditorNormalizedSelectionSet(const UTypedElementSelectionSet& SelectionSet)
{
	const FTypedElementSelectionNormalizationOptions NormalizationOptions = FTypedElementSelectionNormalizationOptions()
		.SetExpandGroups(true)
		.SetFollowAttachment(true);

	return SelectionSet.GetNormalizedSelection(NormalizationOptions);
}

FTypedElementListRef UEditorElementSubsystem::GetEditorManipulableElements(const FTypedElementListRef& NormalizedSelection, UE::Widget::EWidgetMode InManipulationType, UWorld* InRequiredWorld)
{
	 NormalizedSelection->RemoveAll<ITypedElementWorldInterface>([InManipulationType, InRequiredWorld](const TTypedElement<ITypedElementWorldInterface>& InWorldElement)
		{
			return !UEditorElementSubsystem::IsElementEditorManipulable(InWorldElement, InManipulationType, InRequiredWorld);
		});

	 return NormalizedSelection;
}

TTypedElement<ITypedElementWorldInterface> UEditorElementSubsystem::GetLastSelectedEditorManipulableElement(const FTypedElementListRef& NormalizedSelection, UE::Widget::EWidgetMode InManipulationType, UWorld* InRequiredWorld)
{
	return NormalizedSelection->GetBottomElement<ITypedElementWorldInterface>(
		[InManipulationType, InRequiredWorld](const TTypedElement<ITypedElementWorldInterface> Element)
		{
			return UEditorElementSubsystem::IsElementEditorManipulable(Element, InManipulationType, InRequiredWorld);
		});
}

bool UEditorElementSubsystem::IsElementEditorManipulable(const TTypedElement<ITypedElementWorldInterface>& WorldElement, UE::Widget::EWidgetMode InManipulationType, UWorld* InRequiredWorld)
{
	UWorld* OwnerWorld = WorldElement.GetOwnerWorld();
	if (!OwnerWorld)
	{
		return false;
	}

	bool bIsPlayingWorld = OwnerWorld->IsPlayInEditor();

	if (!WorldElement.CanMoveElement(bIsPlayingWorld ? ETypedElementWorldType::Game : ETypedElementWorldType::Editor))
	{
		return false;
	}

	if (InManipulationType == UE::Widget::WM_Scale && !WorldElement.CanScaleElement())
	{
		return false;
	}

	if (InRequiredWorld != nullptr)
	{
		if (OwnerWorld != InRequiredWorld)
		{
			return false;
		}
	}

	return true;
}
