// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/PlacementSubsystem.h"

#include "Containers/Map.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Factories/AssetFactoryInterface.h"
#include "HAL/PlatformCrt.h"
#include "Misc/CoreDelegates.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlacementSubsystem)

class FSubsystemCollectionBase;

void UPlacementSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	FCoreDelegates::GetOnPostEngineInit().AddUObject(this, &UPlacementSubsystem::RegisterPlacementFactories);
	FCoreDelegates::OnEnginePreExit.AddUObject(this, &UPlacementSubsystem::UnregisterPlacementFactories);
}

void UPlacementSubsystem::Deinitialize()
{
	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
}

TArray<FTypedElementHandle> UPlacementSubsystem::PlaceAsset(const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions)
{
	return PlaceAssets(MakeArrayView( {InPlacementInfo} ), InPlacementOptions);
}

TArray<FTypedElementHandle> UPlacementSubsystem::PlaceAssets(TArrayView<const FAssetPlacementInfo> InPlacementInfos, const FPlacementOptions& InPlacementOptions)
{
	TGuardValue<bool> bShouldCreatePreviewElements(bIsCreatingPreviewElements, InPlacementOptions.bIsCreatingPreviewElements);
	TMap<IAssetFactoryInterface*, TArray<FTypedElementHandle>> PlacedElementsByFactory;

	for (const FAssetPlacementInfo& PlacementInfo : InPlacementInfos)
	{
		const FAssetData& AssetData = PlacementInfo.AssetToPlace;
		TScriptInterface<IAssetFactoryInterface> FactoryInterface = PlacementInfo.FactoryOverride;
		if (!FactoryInterface)
		{
			FactoryInterface = FindAssetFactoryFromAssetData(AssetData);
		}

		if (!FactoryInterface || !FactoryInterface->CanPlaceElementsFromAssetData(AssetData))
		{
			continue;
		}

		if (!PlacedElementsByFactory.Contains(&*FactoryInterface))
		{
			FactoryInterface->BeginPlacement(InPlacementOptions);
			PlacedElementsByFactory.Add(&*FactoryInterface);
		}

		FAssetPlacementInfo AdjustedPlacementInfo = PlacementInfo;
		if (!FactoryInterface->PrePlaceAsset(AdjustedPlacementInfo, InPlacementOptions))
		{
			continue;
		}

		TArray<FTypedElementHandle> PlacedHandles = FactoryInterface->PlaceAsset(AdjustedPlacementInfo, InPlacementOptions);
		if (PlacedHandles.Num())
		{
			FactoryInterface->PostPlaceAsset(PlacedHandles, PlacementInfo, InPlacementOptions);
			PlacedElementsByFactory[&*FactoryInterface].Append(MoveTemp(PlacedHandles));
		}
	}

	TArray<FTypedElementHandle> PlacedElements;
	for (TMap<IAssetFactoryInterface*, TArray<FTypedElementHandle>>::ElementType& FactoryAndPlacedElementsPair : PlacedElementsByFactory)
	{
		FactoryAndPlacedElementsPair.Key->EndPlacement(FactoryAndPlacedElementsPair.Value, InPlacementOptions);
		PlacedElements.Append(MoveTemp(FactoryAndPlacedElementsPair.Value));
	}

	return PlacedElements;
}

TScriptInterface<IAssetFactoryInterface> UPlacementSubsystem::FindAssetFactoryFromAssetData(const FAssetData& InAssetData)
{
	for (const TScriptInterface<IAssetFactoryInterface>& AssetFactory : AssetFactories)
	{
		if (AssetFactory && AssetFactory->CanPlaceElementsFromAssetData(InAssetData))
		{
			return AssetFactory;
		}
	}

	return nullptr;
}

bool UPlacementSubsystem::IsCreatingPreviewElements() const
{
	return bIsCreatingPreviewElements;
}

FSimpleMulticastDelegate& UPlacementSubsystem::OnPlacementFactoriesRegistered()
{
	return PlacementFactoriesRegistered;
}

void UPlacementSubsystem::RegisterPlacementFactories()
{
	for (TObjectIterator<UClass> ObjectIt; ObjectIt; ++ObjectIt)
	{
		UClass* TestClass = *ObjectIt;
		if (TestClass->ImplementsInterface(UAssetFactoryInterface::StaticClass()))
		{
			if (!TestClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
			{
				TScriptInterface<IAssetFactoryInterface> NewFactory = NewObject<UObject>(this, TestClass);
				AssetFactories.Add(NewFactory);
			}
		}
	}

	PlacementFactoriesRegistered.Broadcast();
}

// Deprecated in 5.4. This function is a mistake because the user is not expected to pass in a 
// UClass that identifies a subclass of a UClass- they are expected to pass in a UClass that
// identifies their factory (which does not derive from UClass). TSubclassOf does not do any
// verification when it is created, it will just give false in the if statement and the function
// will return nullptr.
TScriptInterface<IAssetFactoryInterface> UPlacementSubsystem::GetAssetFactoryFromFactoryClass(TSubclassOf<UClass> InFactoryInterfaceClass) const
{
	if (InFactoryInterfaceClass)
	{
		for (const TScriptInterface<IAssetFactoryInterface>& AssetFactory : AssetFactories)
		{
			if (AssetFactory.GetObject()->IsA(InFactoryInterfaceClass))
			{
				return AssetFactory;
			}
		}
	}

	return nullptr;
}

TScriptInterface<IAssetFactoryInterface> UPlacementSubsystem::GetAssetFactoryFromFactoryClass(UClass* InFactoryInterfaceClass) const
{
	if (InFactoryInterfaceClass)
	{
		for (const TScriptInterface<IAssetFactoryInterface>& AssetFactory : AssetFactories)
		{
			if (AssetFactory.GetObject()->IsA(InFactoryInterfaceClass))
			{
				return AssetFactory;
			}
		}
	}

	return nullptr;
}

void UPlacementSubsystem::RegisterAssetFactory(TScriptInterface<IAssetFactoryInterface> AssetFactory)
{
	AssetFactories.Add(AssetFactory);
}

void UPlacementSubsystem::UnregisterAssetFactory(TScriptInterface<IAssetFactoryInterface> AssetFactory)
{
	AssetFactories.Remove(AssetFactory);
}

void UPlacementSubsystem::UnregisterPlacementFactories()
{
	AssetFactories.Empty();
}
