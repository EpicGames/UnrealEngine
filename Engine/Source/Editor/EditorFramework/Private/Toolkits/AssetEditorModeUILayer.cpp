// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/AssetEditorModeUILayer.h"

#include "Framework/Docking/WorkspaceItem.h"
#include "HAL/Platform.h"
#include "Misc/Attribute.h"
#include "Textures/SlateIcon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SNullWidget.h"
#include "Toolkits/IToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetEditorModeUILayer)

#define LOCTEXT_NAMESPACE "SLevelEditorToolBox"

const FName UAssetEditorUISubsystem::TopLeftTabID(TEXT("TopLeftModeTab"));
const FName UAssetEditorUISubsystem::BottomLeftTabID(TEXT("BottomLeftModeTab"));
const FName UAssetEditorUISubsystem::TopRightTabID(TEXT("TopRightModeTab"));
const FName UAssetEditorUISubsystem::BottomRightTabID(TEXT("BottomRightModeTab"));
const FName UAssetEditorUISubsystem::VerticalToolbarID = TEXT("VerticalModeToolbar");

FAssetEditorModeUILayer::FAssetEditorModeUILayer(const IToolkitHost* InToolkitHost)
	: ToolkitHost(InToolkitHost)
	, ModeCommands( new FUICommandList() )

{
	RequestedTabInfo.Add(UAssetEditorUISubsystem::VerticalToolbarID, FMinorTabConfig(UAssetEditorUISubsystem::VerticalToolbarID));
	RequestedTabInfo.Add(UAssetEditorUISubsystem::TopLeftTabID, FMinorTabConfig(UAssetEditorUISubsystem::TopLeftTabID));
	RequestedTabInfo.Add(UAssetEditorUISubsystem::BottomLeftTabID, FMinorTabConfig(UAssetEditorUISubsystem::BottomLeftTabID));
	RequestedTabInfo.Add(UAssetEditorUISubsystem::TopRightTabID, FMinorTabConfig(UAssetEditorUISubsystem::TopRightTabID));
	RequestedTabInfo.Add(UAssetEditorUISubsystem::BottomRightTabID, FMinorTabConfig(UAssetEditorUISubsystem::BottomRightTabID));
}

void FAssetEditorModeUILayer::OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit)
{
	GetTabManager()->UnregisterTabSpawner(UAssetEditorUISubsystem::TopLeftTabID);
	GetTabManager()->UnregisterTabSpawner(UAssetEditorUISubsystem::BottomLeftTabID);
	GetTabManager()->UnregisterTabSpawner(UAssetEditorUISubsystem::VerticalToolbarID);
	GetTabManager()->UnregisterTabSpawner(UAssetEditorUISubsystem::TopRightTabID);
	GetTabManager()->UnregisterTabSpawner(UAssetEditorUISubsystem::BottomRightTabID);
}

void FAssetEditorModeUILayer::OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit)
{
	OnToolkitHostShutdownUI.ExecuteIfBound();
	if (HostedToolkit.IsValid() && HostedToolkit.Pin() == Toolkit)
	{
		for (TPair<FName, TWeakPtr<SDockTab>>& SpawnedTab : SpawnedTabs)
		{
			if (SpawnedTab.Value.IsValid())
			{
				SpawnedTab.Value.Pin()->SetContent(SNullWidget::NullWidget);
				SpawnedTab.Value.Pin()->RequestCloseTab();
			}
		}
		for (TPair<FName, FMinorTabConfig>& TabSpawnerInfo : RequestedTabInfo)
		{
			TabSpawnerInfo.Value = FMinorTabConfig(TabSpawnerInfo.Key);
			GetTabManager()->UnregisterTabSpawner(TabSpawnerInfo.Key);
		}
		SpawnedTabs.Empty();
	}
}

TSharedPtr<FTabManager> FAssetEditorModeUILayer::GetTabManager()
{
	return ToolkitHost ? ToolkitHost->GetTabManager() : TSharedPtr<FTabManager>();
}


TSharedPtr<FWorkspaceItem> FAssetEditorModeUILayer::GetModeMenuCategory() const
{
	return TSharedPtr<FWorkspaceItem>();
}

void FAssetEditorModeUILayer::RegisterModeTabSpawners()
{
	RegisterModeTabSpawner(UAssetEditorUISubsystem::TopLeftTabID);
	RegisterModeTabSpawner(UAssetEditorUISubsystem::BottomLeftTabID);
	RegisterModeTabSpawner(UAssetEditorUISubsystem::VerticalToolbarID);
	RegisterModeTabSpawner(UAssetEditorUISubsystem::TopRightTabID);
	RegisterModeTabSpawner(UAssetEditorUISubsystem::BottomRightTabID);

}

void FAssetEditorModeUILayer::RegisterModeTabSpawner(const FName TabID)
{
	TSharedRef<FWorkspaceItem> MenuGroup = GetModeMenuCategory().ToSharedRef();
	bool bShowMenuOption = GetStoredSpawner(TabID).IsBound();
	GetTabManager()->RegisterTabSpawner(TabID,
		FOnSpawnTab::CreateSP(this, &FAssetEditorModeUILayer::SpawnStoredTab, TabID),
		FCanSpawnTab::CreateSP(this, &FAssetEditorModeUILayer::CanSpawnStoredTab, TabID))
		.SetDisplayNameAttribute(MakeAttributeSP(this, &FAssetEditorModeUILayer::GetTabSpawnerName, TabID))
		.SetTooltipTextAttribute(MakeAttributeSP(this, &FAssetEditorModeUILayer::GetTabSpawnerTooltip, TabID))
		.SetIcon(GetTabSpawnerIcon(TabID))
		.SetAutoGenerateMenuEntry(bShowMenuOption)
		.SetGroup(MenuGroup);
}

void FAssetEditorModeUILayer::SetModePanelInfo(const FName InTabSpawnerID, const FMinorTabConfig& InTabInfo)
{
	RequestedTabInfo.Emplace(InTabSpawnerID, InTabInfo);
}

void FAssetEditorModeUILayer::SetSecondaryModeToolbarName(FName InName)
{
	SecondaryModeToolbarName = InName;
}

TMap<FName, TWeakPtr<SDockTab>> FAssetEditorModeUILayer::GetSpawnedTabs()
{
	return SpawnedTabs;
}

const FOnSpawnTab& FAssetEditorModeUILayer::GetStoredSpawner(const FName TabID)
{
	return RequestedTabInfo[TabID].OnSpawnTab;
}

TSharedRef<SDockTab> FAssetEditorModeUILayer::SpawnStoredTab(const FSpawnTabArgs& Args, const FName TabID)
{
	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab);
	if (GetStoredSpawner(TabID).IsBound())
	{
		FOnSpawnTab& StoredTabSpawner = RequestedTabInfo[TabID].OnSpawnTab;
		SpawnedTab = StoredTabSpawner.Execute(Args);
	}
	SpawnedTabs.Emplace(TabID, SpawnedTab);
	return SpawnedTab;
}

bool FAssetEditorModeUILayer::CanSpawnStoredTab(const FSpawnTabArgs& Args, const FName TabID)
{
	bool bCanSpawnTab = GetStoredSpawner(TabID).IsBound();
	if (RequestedTabInfo[TabID].CanSpawnTab.IsBound())
	{
		bCanSpawnTab |= RequestedTabInfo[TabID].CanSpawnTab.Execute(Args);
	}
	return bCanSpawnTab;
}

FText FAssetEditorModeUILayer::GetTabSpawnerName(const FName TabID) const
{
	if (!RequestedTabInfo[TabID].TabLabel.IsEmpty())
	{
		return RequestedTabInfo[TabID].TabLabel;
	}
	return FText::GetEmpty();
}

FText FAssetEditorModeUILayer::GetTabSpawnerTooltip(const FName TabID) const
{
	if (!RequestedTabInfo[TabID].TabTooltip.IsEmpty())
	{
		return RequestedTabInfo[TabID].TabTooltip;
	}
	return FText::GetEmpty();
}

const FSlateIcon& FAssetEditorModeUILayer::GetTabSpawnerIcon(const FName TabID) const
{
	if (RequestedTabInfo[TabID].TabIcon.IsSet())
	{
		return RequestedTabInfo[TabID].TabIcon;
	}

	return GetModeMenuCategory()->GetIcon();
}

#undef LOCTEXT_NAMESPACE

