// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/ToolkitManager.h"

#include "Delegates/Delegate.h"
#include "Framework/Docking/TabManager.h"
#include "HAL/PlatformCrt.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Toolkits/IToolkit.h"
#include "Toolkits/IToolkitHost.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"

class UObject;

static TSharedRef<SDockTab> SpawnStandaloneToolkitHost( const FSpawnTabArgs& Args )
{
	return
		SNew(SDockTab)
		.TabRole(ETabRole::MajorTab)
		.ContentPadding(FMargin(0));
}

static TSharedPtr<SDockTab> NeverReuse(const FTabId&)
{
	return TSharedPtr<SDockTab>();
}

FToolkitManager& FToolkitManager::Get()
{
	static TSharedRef< FToolkitManager > ToolkitManagerInstance( new FToolkitManager() );
	return *ToolkitManagerInstance;
}

FToolkitManager::FToolkitManager()
{
	FGlobalTabmanager::Get()->RegisterTabSpawner( "StandaloneToolkit", FOnSpawnTab::CreateStatic( &SpawnStandaloneToolkitHost ) )
		.SetReuseTabMethod(FOnFindTabToReuse::CreateStatic( &NeverReuse ) )
		.SetAutoGenerateMenuEntry(false);
}


void FToolkitManager::RegisterNewToolkit( const TSharedRef< IToolkit > NewToolkit )
{
	// Add to our list
	check( !Toolkits.Contains( NewToolkit ) );
	Toolkits.Add( NewToolkit );

	// Tell the host about this tool kit
	const TSharedPtr< IToolkitHost > ToolkitHost = NewToolkit->GetToolkitHost();
	if( ToolkitHost.IsValid() )
	{
		ToolkitHost->OnToolkitHostingStarted( NewToolkit );
	}
}


void FToolkitManager::CloseToolkit( const TSharedRef< IToolkit > ClosingToolkit )
{
	// NOTE: This function is called when a user closes a toolkit interactively while the toolkit's host is still
	//       around.  If the host itself is closed (such as the SLevelEditor goes away during shutdown), then this
	//       function will not be called.  The toolkit will instead be cleaned up on OnToolkitHostDestroyed, below.

	// Tell the host about this tool kit
	if( ClosingToolkit->IsHosted() )
	{
		const TSharedPtr< IToolkitHost > ToolkitHost = ClosingToolkit->GetToolkitHost();
		if( ToolkitHost.IsValid() )
		{
			ToolkitHost->OnToolkitHostingFinished( ClosingToolkit );
		}
	}

	// Remove from our list
	Toolkits.Remove( ClosingToolkit );
}



void FToolkitManager::OnToolkitHostDestroyed( IToolkitHost* HostBeingDestroyed )
{
	for( auto ToolkitIndex = 0; ToolkitIndex < Toolkits.Num(); ++ToolkitIndex )
	{
		TSharedPtr< IToolkit > Toolkit = Toolkits[ ToolkitIndex ];
		check( Toolkit.IsValid() );

		if( !Toolkit->IsHosted() || TSharedPtr< IToolkitHost >( Toolkit->GetToolkitHost() ).Get() == HostBeingDestroyed )
		{
			// Found a toolkit that is hosted by the host that is going away.  We'll destroy this toolkit now.
			// NOTE: In this case, OnToolkitHostingFinished() is not called on the host (since it's probably in
			//       the middle of being destructed!)
			Toolkits.RemoveAt( ToolkitIndex-- );

			// Continue iterating, so we can cleanup any references to toolkits whose host is no
			// longer valid.
		}
	}
}


TSharedPtr< IToolkit > FToolkitManager::FindEditorForAsset( const UObject* Asset )
{
	for( auto ToolkitIndex = 0; ToolkitIndex < Toolkits.Num(); ++ToolkitIndex )
	{
		TSharedPtr< IToolkit > Toolkit = Toolkits[ ToolkitIndex ];
		check( Toolkit.IsValid() );

		if( Toolkit->IsAssetEditor() )
		{
			if (Toolkit->GetObjectsCurrentlyBeingEdited()->FindByKey(Asset) != NULL )
			{
				return Toolkit;
			}
		}
	}

	return NULL;
}
