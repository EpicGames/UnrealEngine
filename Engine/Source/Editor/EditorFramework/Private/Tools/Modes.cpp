// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/Modes.h"

#include "Containers/UnrealString.h"
#include "UObject/UnrealNames.h"

FEditorModeInfo::FEditorModeInfo()
	: ID(NAME_None)
	, PriorityOrder(MAX_int32)
	, Visibility(false)
{
}

FEditorModeInfo::FEditorModeInfo(
	FEditorModeID InID,
	FText InName,
	FSlateIcon InIconBrush,
	TAttribute<bool> InVisibility,
	int32 InPriorityOrder
	)
	: ID(InID)
	, ToolbarCustomizationName(*(InID.ToString() + TEXT("Toolbar")))
	, Name(InName)
	, IconBrush(InIconBrush)
	, PriorityOrder(InPriorityOrder)
	, Visibility(InVisibility)
{
}

bool FEditorModeInfo::IsVisible() const
{
	return Visibility.Get(false);
}