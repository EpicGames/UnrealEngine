// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportTabContent.h"

#include "EditorViewportLayout.h"
#include "Templates/Tuple.h"


// FViewportTabContent ///////////////////////////

bool FViewportTabContent::IsVisible() const
{
	if (ActiveViewportLayout.IsValid())
	{
		return ActiveViewportLayout->IsVisible();
	}
	return false;
}

bool FViewportTabContent::BelongsToTab(TSharedRef<class SDockTab> InParentTab) const
{
	TSharedPtr<SDockTab> ParentTabPinned = ParentTab.Pin();
	return ParentTabPinned == InParentTab;
}

bool FViewportTabContent::IsViewportConfigurationSet(const FName& ConfigurationName) const
{
	if (ActiveViewportLayout.IsValid())
	{
		return ActiveViewportLayout->GetActivePaneConfigurationTypeName() == ConfigurationName;
	}
	return false;
}

const TMap< FName, TSharedPtr< IEditorViewportLayoutEntity > >* FViewportTabContent::GetViewports() const
{
	if (ActiveViewportLayout.IsValid())
	{
		return &ActiveViewportLayout->GetViewports();
	}
	return nullptr;
}

void FViewportTabContent::PerformActionOnViewports(ViewportActionFunction& TFuncPtr)
{
	const TMap< FName, TSharedPtr<IEditorViewportLayoutEntity> >* Entities = GetViewports();
	if (!Entities)
	{
		return;
	}

	for (auto& Entity : *Entities)
	{
		TFuncPtr(Entity.Key, Entity.Value);
	}
}
