// Copyright Epic Games, Inc. All Rights Reserved.

#include "Viewports/InViewportUIDragOperation.h"

#include "Framework/Application/SlateApplication.h"
#include "Layout/SlateRect.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SWindow.h"

class SWidget;
struct FPointerEvent;

const FVector2f FInViewportUIDragOperation::GetDecoratorOffsetFromCursor()
{
	return DecoratorOffsetFromCursor;
}

void FInViewportUIDragOperation::BroadcastDropEvent(const FVector2D InLocation)
{
	DropDelegate.ExecuteIfBound(InLocation);
}

void FInViewportUIDragOperation::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
{
	check(CursorDecoratorWindow.IsValid());

	// Destroy the CursorDecoratorWindow by calling the base class implementation because we are relocating the content into a more permanent home.
	FDragDropOperation::OnDrop(bDropWasHandled, MouseEvent);

	UIBeingDragged.Reset();
}

void FInViewportUIDragOperation::OnDragged(const FDragDropEvent& DragDropEvent)
{
	// The tab is being dragged. Move the the decorator window to match the cursor position.
	FVector2f TargetPosition = DragDropEvent.GetScreenSpacePosition() - GetDecoratorOffsetFromCursor();
	CursorDecoratorWindow->UpdateMorphTargetShape(FSlateRect(TargetPosition.X, TargetPosition.Y, TargetPosition.X + LastContentSize.X, TargetPosition.Y + LastContentSize.Y));
	CursorDecoratorWindow->MoveWindowTo(TargetPosition);
}

TSharedRef<FInViewportUIDragOperation> FInViewportUIDragOperation::New(const TSharedRef<SWidget>& InUIToBeDragged, const FVector2D InDecoratorOffset, const FVector2D& OwnerAreaSize, FOnInViewportUIDropped& InDropDelegate)
{
	const TSharedRef<FInViewportUIDragOperation> Operation = MakeShareable(new FInViewportUIDragOperation(InUIToBeDragged, InDecoratorOffset, OwnerAreaSize, InDropDelegate));
	return Operation;
}

FInViewportUIDragOperation::~FInViewportUIDragOperation()
{

}

FInViewportUIDragOperation::FInViewportUIDragOperation(const TSharedRef<class SWidget>& InUIToBeDragged, const FVector2D InDecoratorOffset, const FVector2D& OwnerAreaSize, FOnInViewportUIDropped& InDropDelegate)
	: UIBeingDragged(InUIToBeDragged)
	, DecoratorOffsetFromCursor(UE::Slate::CastToVector2f(InDecoratorOffset))
	, LastContentSize(UE::Slate::CastToVector2f(OwnerAreaSize))
	, DropDelegate(InDropDelegate)
{
	// Create the decorator window that we will use during this drag and drop to make the user feel like
	// they are actually dragging a piece of UI.

	// Start the window off hidden.
	const bool bShowImmediately = true;
	CursorDecoratorWindow = FSlateApplication::Get().AddWindow(SWindow::MakeStyledCursorDecorator(FAppStyle::Get().GetWidgetStyle<FWindowStyle>("InViewportDecoratorWindow")), bShowImmediately);
	CursorDecoratorWindow->SetOpacity(0.45f);
	CursorDecoratorWindow->SetContent
	(
		InUIToBeDragged
	);

}