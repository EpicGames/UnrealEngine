// Copyright Epic Games, Inc. All Rights Reserved.

#include "Viewports/SDepthBar.h"
#include "Math/UnitConversion.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Layout/ArrangedChildren.h"

#define LOCTEXT_NAMESPACE "DepthBar"

namespace UE::EditorWidgets::Private
{
	constexpr double MinIndicatorDifference = 2.0;
	
	FText GetDistanceDisplayText(double Value)
	{
		if (FUnitConversion::Settings().ShouldDisplayUnits())
		{
			const TArray<EUnit>& AllDistanceUnits = FUnitConversion::Settings().GetDisplayUnits(EUnitType::Distance);
			const EUnit BaseUnit = AllDistanceUnits.Num() ? AllDistanceUnits[0] : EUnit::Centimeters;
			const FNumericUnit DisplayUnit = FUnitConversion::QuantizeUnitsToBestFit(Value, BaseUnit);
			
			const FNumberFormattingOptions Options = FNumberFormattingOptions()
				.SetMaximumFractionalDigits(2)
				.SetMinimumFractionalDigits(2);
			
			return FText::Format(
				LOCTEXT("DistanceFormat", "{0}{1}"),
				FText::AsNumber(DisplayUnit.Value, &Options),
				FText::FromString(FUnitConversion::GetUnitDisplayString(DisplayUnit.Units))
			);
		}
		return FText::Format(LOCTEXT("RawDistanceFormat", "{0}"), Value);
	}
}

void SDepthBar::Construct(const FArguments& InArgs)
{
	Mode = InArgs._Mode;
	
	DepthSpace = InArgs._DepthSpace;
	NearPlane = InArgs._NearPlane;
	FarPlane = InArgs._FarPlane;
	OnNearPlaneChanged = InArgs._OnNearPlaneChanged;
	OnFarPlaneChanged = InArgs._OnFarPlaneChanged;

	if (InArgs._Style)
	{
		Style = InArgs._Style;
	}
	else
	{
		Style = &FDepthBarStyle::GetDefault();
	}

	DepthIndicatorChildren.AddSlots(MoveTemp(const_cast<TArray<FDepthIndicatorSlot::FSlotArguments>&>(InArgs._Slots)));
	
	AddDepthLabelSlot()
	.VAlign(VAlign_Top)
	.OnGetBarPosition_Lambda([](const FBarPositions& BarPositions) { return static_cast<float>(BarPositions.BarFarPlane); })
	[
		SNew(SBorder)
		.BorderImage(&Style->BackgroundBrush)
		.Padding(6.0f, 2.0f)
		.Visibility(this, &SDepthBar::GetFarPlaneLabelVisibility)
		[
			SNew(STextBlock)
			.Text(this, &SDepthBar::GetFarPlaneText)
		]
	];
	
	AddDepthLabelSlot()
	.VAlign(VAlign_Bottom)
	.OnGetBarPosition_Lambda([](const FBarPositions& BarPositions) { return static_cast<float>(BarPositions.BarNearPlane); })
	[
		SNew(SBorder)
		.BorderImage(&Style->BackgroundBrush)
		.Padding(6.0f, 2.0f)
		.Visibility(this, &SDepthBar::GetNearPlaneLabelVisibility)
		[
			SNew(STextBlock)
			.Text(this, &SDepthBar::GetNearPlaneText)
		]
	];
}

FCursorReply SDepthBar::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	using namespace UE::EditorWidgets::Private;
	
	switch (DragTarget)
	{
		case EDragTarget::Far_Handle:
		case EDragTarget::Near_Handle:
			return FCursorReply::Cursor(EMouseCursor::ResizeUpDown);
		case EDragTarget::Slice:
			return FCursorReply::Cursor(EMouseCursor::GrabHandClosed);
		default:
			switch (HoverTarget)
			{
				case EDragTarget::Far_Handle:
				case EDragTarget::Near_Handle:
					return FCursorReply::Cursor(EMouseCursor::ResizeUpDown);
				case EDragTarget::Slice:
					return FCursorReply::Cursor(EMouseCursor::GrabHand);
				default:
					return FCursorReply::Cursor(EMouseCursor::Default);	
			}
	}
}

SDepthBar::FDepthIndicatorSlot& SDepthBar::GetDepthIndicatorSlot(int32 Index)
{
	return DepthIndicatorChildren[Index];
}

const SDepthBar::FDepthIndicatorSlot& SDepthBar::GetDepthIndicatorSlot(int32 Index) const
{
	return DepthIndicatorChildren[Index];
}

void SDepthBar::ClearDepthIndicators()
{
	DepthIndicatorChildren.Empty();
}

FReply SDepthBar::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	using namespace UE::EditorWidgets::Private;

	if (HoverTarget != EDragTarget::None)
	{
		TOptional<FBarPositions> OptionalPositions = GetBarPositions(MyGeometry); 
		if (const FBarPositions* Positions = OptionalPositions.GetPtrOrNull())
		{
			const FVector2D MousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		
			DragTarget = HoverTarget;
			
			switch (DragTarget)
			{
			case EDragTarget::Far_Handle:
			case EDragTarget::Slice:
				DragPosition = FVector2D(0.0, Positions->BarFarPlane);
				break;
			case EDragTarget::Near_Handle:
				DragPosition = FVector2D(0.0, Positions->BarNearPlane);
				break;
			default:
				DragPosition = MousePosition;
				break;
			}
			
			StartDragPosition = DragPosition;
			
			return FReply::Handled().CaptureMouse(AsShared());
		}
	}

	return FReply::Unhandled();
}

FReply SDepthBar::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	using namespace UE::EditorWidgets::Private;

	TOptional<FBarPositions> OptionalPositions = GetBarPositions(MyGeometry);

	if (DragTarget != EDragTarget::None)
	{
		if (const FBarPositions* Positions = OptionalPositions.GetPtrOrNull())
		{
			double DragRatio = 1.0;
			
			if (MouseEvent.IsAltDown())
			{
				// Holding alt down should give consistent world-space movement (1px = 1u), but not if doing so would _lose_ precision.
				DragRatio = FMath::Min(1.0, (Positions->BarMax - Positions->BarMin) / (Positions->DepthSpace.GetAlignedMax() - Positions->DepthSpace.GetAlignedMin()));
			}
			
			if (MouseEvent.IsControlDown())
			{
				DragRatio *= 0.1;
			}
			
			// Use doubles for higher precision with very large worlds
			DragPosition += FVector2D(MouseEvent.GetCursorDelta()) * DragRatio;
		
			if (Mode.Get() == EMode::Orthographic)
			{
				if (DragTarget == EDragTarget::Far_Handle)
				{
					const double ClampedBarPosition = FMath::Clamp(DragPosition.Y, Positions->BarMin, Positions->BarNearPlane - MinIndicatorDifference);
					
					if (ClampedBarPosition == Positions->BarMin)
					{
						OnFarPlaneChanged.ExecuteIfBound(TOptional<double>());
					}
					else
					{
						OnFarPlaneChanged.ExecuteIfBound(Positions->BarToAlignedPosition(ClampedBarPosition));
					}
					
					return FReply::Handled();
				}
				if (DragTarget == EDragTarget::Near_Handle)
				{
					const double ClampedBarPosition = FMath::Clamp(DragPosition.Y, Positions->BarFarPlane + MinIndicatorDifference, Positions->BarMax);
					if (ClampedBarPosition == Positions->BarMax)
					{
						OnNearPlaneChanged.ExecuteIfBound(TOptional<double>());
					}
					else
					{
						OnNearPlaneChanged.ExecuteIfBound(Positions->BarToAlignedPosition(ClampedBarPosition));
					}
					
					return FReply::Handled();
				}
				if (DragTarget == EDragTarget::Slice)
				{
					const double ClampedBarPosition = FMath::Clamp(DragPosition.Y, Positions->BarMin, Positions->BarMax - (Positions->BarNearPlane - Positions->BarFarPlane));
					
					OnFarPlaneChanged.ExecuteIfBound(Positions->BarToAlignedPosition(ClampedBarPosition));
					OnNearPlaneChanged.ExecuteIfBound(Positions->BarToAlignedPosition(ClampedBarPosition + (Positions->BarNearPlane - Positions->BarFarPlane)));
					
					return FReply::Handled();
				}
			}
		}
	}
	else if (const FBarPositions* Positions = OptionalPositions.GetPtrOrNull())
	{
		const FVector2f MousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		
		const float SliderHeight = Style->SliceHandleSize.Y;
		
		const FMargin Inset = GetBarInset();
		
		if (MousePosition.Y > Style->Padding.Top && MousePosition.Y < Style->Padding.Top + Style->FarPlaneButtonPadding.GetTotalSpaceAlong<Orient_Vertical>() + Style->PlaneButtonSize.Y)
		{
			HoverTarget = EDragTarget::Far_Button;
		}
		else if (MousePosition.Y >= Positions->BarFarPlane - SliderHeight && MousePosition.Y < Positions->BarFarPlane)
		{
			HoverTarget = EDragTarget::Far_Handle; 
		}
		else if (MousePosition.Y >= Positions->BarNearPlane && MousePosition.Y < Positions->BarNearPlane + SliderHeight)
		{
			HoverTarget = EDragTarget::Near_Handle;
		}
		else if (MousePosition.Y > Positions->BarFarPlane && MousePosition.Y < Positions->BarNearPlane)
		{
			HoverTarget = EDragTarget::Slice;
		}
		else if (MousePosition.Y > MyGeometry.GetLocalSize().Y - Inset.Bottom)
		{
			HoverTarget = EDragTarget::Near_Button;
		}
		else
		{
			HoverTarget = EDragTarget::None;
		}
	}
	return FReply::Unhandled();
}

FReply SDepthBar::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (DragTarget != EDragTarget::None)
	{
		constexpr float ClickThreshold = 3.0f;
		if (FVector2D::DistSquared(DragPosition, StartDragPosition) < ClickThreshold)
		{
			if (DragTarget == EDragTarget::Far_Button)
			{
				OnFarPlaneChanged.ExecuteIfBound(TOptional<double>());
			}
			if (DragTarget == EDragTarget::Near_Button)
			{
				OnNearPlaneChanged.ExecuteIfBound(TOptional<double>());
			}
		}
	
		DragTarget = EDragTarget::None;
		HoverTarget = EDragTarget::None;
		return FReply::Handled().ReleaseMouseCapture();
	}
	return FReply::Unhandled();
}

FReply SDepthBar::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (HoverTarget == EDragTarget::Far_Button)
	{
		OnFarPlaneChanged.ExecuteIfBound(TOptional<double>());
		return FReply::Handled().ReleaseMouseCapture();
	}
	if (HoverTarget == EDragTarget::Near_Button)
	{
		OnNearPlaneChanged.ExecuteIfBound(TOptional<double>());
		return FReply::Handled().ReleaseMouseCapture();
	}
	return FReply::Unhandled();
}

void SDepthBar::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SPanel::OnMouseLeave(MouseEvent);
	
	if (DragTarget == EDragTarget::None)
	{
		HoverTarget = EDragTarget::None;
	}
}

void SDepthBar::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	TOptional<FBarPositions> OptionalPositions = GetBarPositions(AllottedGeometry); 
	if (FBarPositions* BarPositions = OptionalPositions.GetPtrOrNull())
	{
		FMargin Inset = GetBarInset();
		const FVector2f Center = AllottedGeometry.GetLocalSize() * 0.5f;
		
		for (int32 ChildIndex = 0; ChildIndex < DepthIndicatorChildren.Num(); ++ChildIndex)
		{
			const FDepthIndicatorSlot& CurrentChild = GetDepthIndicatorSlot(ChildIndex);
			
			if (CurrentChild.GetBounds().IsSet())
			{
				FBoxSphereBounds Bounds = CurrentChild.GetBounds().Get();
				const FSphere Sphere = Bounds.GetSphere();
				
				const FVector Max = Sphere.Center + BarPositions->DepthSpace.GetForward() * Sphere.W;
				const FVector Min = Sphere.Center - BarPositions->DepthSpace.GetForward() * Sphere.W;
					
				const float BarMax = static_cast<float>(BarPositions->WorldToBarPosition(Max));
				const float BarMin = static_cast<float>(BarPositions->WorldToBarPosition(Min));
				const float BarSize = FMath::Max(BarMin - BarMax, 1.0f);
				
				FVector2f Offset(Inset.Left, 0.0f);
				FVector2f BoundSize(AllottedGeometry.GetLocalSize().X - Inset.GetTotalSpaceAlong<Orient_Horizontal>(), BarSize);
				
				FVector2f ChildDesiredSize = CurrentChild.GetWidget()->GetDesiredSize();
				
				const EHorizontalAlignment HorizontalAlignment = CurrentChild.GetHorizontalAlignment();
				switch (HorizontalAlignment)
				{
				case HAlign_Left:
				case HAlign_Fill:
					Offset.X = Inset.Left;
					break;
				case HAlign_Right:
					Offset.X = Inset.Right - ChildDesiredSize.X;
					break;
				case HAlign_Center:
					Offset.X = Center.X - ChildDesiredSize.X * 0.5f;
					break;
				}
				
				const EVerticalAlignment VerticalAlignment = CurrentChild.GetVerticalAlignment(); 
				switch (VerticalAlignment)
				{
				case VAlign_Top:
					Offset.Y = BarMax - ChildDesiredSize.Y;
					break;
				case VAlign_Bottom:
					Offset.Y = BarMin;
					break;
				case VAlign_Center:
					Offset.Y = BarMax + (BarMin - BarMax) * 0.5f - ChildDesiredSize.Y * 0.5f;
					break;
				case VAlign_Fill:
					Offset.Y = BarMax;
					break;
				}
				
				// Allow minimum child size to overstep the bounds
				if (ChildDesiredSize.X > BoundSize.X)
				{
					Offset.X -= (ChildDesiredSize.X - BoundSize.X) * 0.5f;
					BoundSize.X = ChildDesiredSize.X;
				}
				else if (HorizontalAlignment != HAlign_Fill)
				{
					BoundSize.X = ChildDesiredSize.X;
				}
				
				if (ChildDesiredSize.Y > BoundSize.Y)
				{
					Offset.Y -= (ChildDesiredSize.Y - BoundSize.Y) * 0.5f;
					BoundSize.Y = ChildDesiredSize.Y;
				}
				else if (VerticalAlignment != VAlign_Fill)
				{
					BoundSize.Y = ChildDesiredSize.Y;
				}
				
				ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(CurrentChild.GetWidget(), BoundSize, FSlateLayoutTransform(Offset)));
			}
		}
		
		for (int32 ChildIndex = 0; ChildIndex < DepthLabelChildren.Num(); ++ChildIndex)
		{
			const FDepthLabelSlot& CurrentChild = DepthLabelChildren[ChildIndex];
			
			if (CurrentChild.GetWidget()->GetVisibility() != EVisibility::Visible)
			{
				continue;
			}
			
			const float BarPosition = CurrentChild.GetBarPosition(*BarPositions);
			
			FVector2f ChildDesiredSize = CurrentChild.GetWidget()->GetDesiredSize();
			FVector2f Offset(-ChildDesiredSize.X - 12.0f, BarPosition);
			
			switch (CurrentChild.GetVerticalAlignment())
			{
			case VAlign_Top:
				Offset.Y -= ChildDesiredSize.Y;
				break;
			case VAlign_Center:
			case VAlign_Fill:
				Offset.Y -= ChildDesiredSize.Y * 0.5f;
				break;
			default:
				break;
			}
			
			ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(
				CurrentChild.GetWidget(),
				Offset,
				ChildDesiredSize
			));
		}
	}	
}

FChildren* SDepthBar::GetChildren()
{
	return &AllChildren;
}

int32 SDepthBar::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	using namespace UE::EditorWidgets::Private;

	const FVector2f Center = AllottedGeometry.GetLocalSize() * 0.5f;

	// Background
	FSlateDrawElement::MakeBox(OutDrawElements,
		LayerId++,
		AllottedGeometry.ToPaintGeometry(),
		&Style->BackgroundBrush,
		ESlateDrawEffect::None,
		Style->BackgroundBrush.GetTint(InWidgetStyle)
	);
	
	// Plane Buttons
	{
		const float PlaneButtonHorizontalInset = Center.X - Style->PlaneButtonSize.X * 0.5f;  
	
		FSlateDrawElement::MakeBox(OutDrawElements,
			LayerId,
			AllottedGeometry.MakeChild(
				FVector2f(Style->PlaneButtonSize),
				FSlateLayoutTransform(FVector2f(
					PlaneButtonHorizontalInset,
					Style->Padding.Top + Style->FarPlaneButtonPadding.Top
				))
			).ToPaintGeometry(),
			&Style->FarPlaneButtonBrush,
			ESlateDrawEffect::None,
			(HoverTarget == EDragTarget::Far_Button ? Style->PlaneButtonHoveredColor : Style->PlaneButtonNormalColor).GetColor(InWidgetStyle)
		);
	
		FSlateDrawElement::MakeBox(OutDrawElements,
			LayerId,
			AllottedGeometry.MakeChild(
				FVector2f(Style->PlaneButtonSize),
				FSlateLayoutTransform(FVector2f(
					PlaneButtonHorizontalInset,
					AllottedGeometry.GetLocalSize().Y - Style->PlaneButtonSize.Y - Style->Padding.Bottom - Style->NearPlaneButtonPadding.Bottom
				))
			).ToPaintGeometry(),
			&Style->NearPlaneButtonBrush,
			ESlateDrawEffect::None,
			(HoverTarget == EDragTarget::Near_Button ? Style->PlaneButtonHoveredColor : Style->PlaneButtonNormalColor).GetColor(InWidgetStyle)
		);
	}
	
	// Bar slider
	TOptional<FBarPositions> OptionalPositions = GetBarPositions(AllottedGeometry); 
	if (const FBarPositions* BarPositions = OptionalPositions.GetPtrOrNull())
	{
		// Truncate double values for slate
		const float BarFarPlane = static_cast<float>(BarPositions->BarFarPlane);
		const float BarNearPlane = static_cast<float>(BarPositions->BarNearPlane);
		const float BarMin = static_cast<float>(BarPositions->BarMin);
		const float BarMax = static_cast<float>(BarPositions->BarMax);
	
		// Draw the track
		FSlateDrawElement::MakeBox(OutDrawElements,
			LayerId++,
			AllottedGeometry.MakeChild(
				FVector2f(Style->TrackWidth, BarMax - BarMin + Style->SliceHandleSize.Y * 2.0f),
				FSlateLayoutTransform(FVector2f(
					Center.X - Style->TrackWidth * 0.5f,
					BarMin - Style->SliceHandleSize.Y
				))
			).ToPaintGeometry(),
			&Style->TrackBrush,
			ESlateDrawEffect::None,
			Style->TrackBrush.GetTint(InWidgetStyle)
		);

		if (HoverTarget == EDragTarget::Slice)
		{
			LayerId = SPanel::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled) + 1;
		}
	
		const FSlateBrush* CurrentSliceBrush = HoverTarget == EDragTarget::Slice ? &Style->SliceHoveredBrush : &Style->SliceNormalBrush;
		FLinearColor SliceTint = CurrentSliceBrush->GetTint(InWidgetStyle);
		if (HoverTarget == EDragTarget::Slice)
		{
			SliceTint.A = 0.5f;
		}

		// Draw the slice
		FSlateDrawElement::MakeBox(OutDrawElements,
			LayerId,
			AllottedGeometry.MakeChild(
				FVector2f(Style->SliceWidth, BarNearPlane - BarFarPlane),
				FSlateLayoutTransform(FVector2f(Center.X - Style->SliceWidth * 0.5f, BarFarPlane))
			).ToPaintGeometry(),
			CurrentSliceBrush,
			ESlateDrawEffect::None,
			SliceTint
		);
		
		if (HoverTarget != EDragTarget::Slice)
		{
			LayerId = SPanel::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled) + 1;
		}
		
		// Draw the slice handles
		const FSlateBrush* TopBrush = HoverTarget == EDragTarget::Far_Handle ? &Style->SliceTopHoveredBrush : &Style->SliceTopBrush;
		FSlateDrawElement::MakeBox(OutDrawElements,
			++LayerId,
			AllottedGeometry.MakeChild(
				FVector2f(Style->SliceHandleSize),
				FSlateLayoutTransform(FVector2f(Center.X - Style->SliceHandleSize.X * 0.5f, BarFarPlane - Style->SliceHandleSize.Y))
			).ToPaintGeometry(),
			TopBrush,
			ESlateDrawEffect::None,
			TopBrush->GetTint(InWidgetStyle)
		);
		
		const FSlateBrush* BottomBrush = HoverTarget == EDragTarget::Near_Handle ? &Style->SliceBottomHoveredBrush : &Style->SliceBottomBrush;
		FSlateDrawElement::MakeBox(OutDrawElements,
			++LayerId,
			AllottedGeometry.MakeChild(
				FVector2f(Style->SliceHandleSize),
				FSlateLayoutTransform(FVector2f(Center.X - Style->SliceHandleSize.X * 0.5f, BarNearPlane))
			).ToPaintGeometry(),
			BottomBrush,
			ESlateDrawEffect::None,
			BottomBrush->GetTint(InWidgetStyle)
		);
		
		// Draw the "shades" over out-of-slice content
		if (BarMin < BarFarPlane - Style->SliceHandleSize.Y)
		{
			FSlateDrawElement::MakeBox(OutDrawElements,
				LayerId,
				AllottedGeometry.MakeChild(
					FVector2f(AllottedGeometry.GetLocalSize().X, BarFarPlane - Style->SliceHandleSize.Y - BarMin),
					FSlateLayoutTransform(FVector2f(0.0f, BarMin)
				)).ToPaintGeometry(),
				&Style->BackgroundBrush,
				ESlateDrawEffect::None,
				Style->BackgroundBrush.GetTint(InWidgetStyle).CopyWithNewOpacity(0.5f)
			);
		}
		
		if (BarMax > BarNearPlane + Style->SliceHandleSize.Y)
		{
			FSlateDrawElement::MakeBox(OutDrawElements,
				LayerId,
				AllottedGeometry.MakeChild(
					FVector2f(AllottedGeometry.GetLocalSize().X, BarMax - BarNearPlane - Style->SliceHandleSize.Y),
					FSlateLayoutTransform(FVector2f(0.0f, BarNearPlane + Style->SliceHandleSize.Y)
				)).ToPaintGeometry(),
				&Style->BackgroundBrush,
				ESlateDrawEffect::None,
				Style->BackgroundBrush.GetTint(InWidgetStyle).CopyWithNewOpacity(0.5f)
			);
		}
	}

	return LayerId;
}

FVector2D SDepthBar::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	return FVector2D(Style->SliceWidth + Style->Padding.GetTotalSpaceAlong<Orient_Horizontal>(), 800.0);
}

FMargin SDepthBar::GetBarInset() const
{
	FMargin Inset = Style->Padding;
	
	// Allow space for the plane buttons & their paddings
	Inset.Top += Style->PlaneButtonSize.Y;
	Inset.Bottom += Style->PlaneButtonSize.Y;
	Inset.Top += Style->FarPlaneButtonPadding.GetTotalSpaceAlong<Orient_Vertical>();
	Inset.Bottom += Style->NearPlaneButtonPadding.GetTotalSpaceAlong<Orient_Vertical>();
	
	// Allow space for the top & bottom handles
	Inset.Top += Style->SliceHandleSize.X;
	Inset.Bottom += Style->SliceHandleSize.Y;
	
	return Inset;
}

TOptional<SDepthBar::FBarPositions> SDepthBar::GetBarPositions(const FGeometry& InGeometry) const
{
	TOptional<FDepthSpace> Space = DepthSpace.Get();
	if (!Space.IsSet())
	{
		return TOptional<FBarPositions>();
	}
	
	const FMargin Inset = GetBarInset();
	
	if (Mode.Get() == EMode::Orthographic)
	{
		FBarPositions Positions
		{
			.DepthSpace = Space.GetValue(),
			.BarMin = Inset.Top,
			.BarMax = InGeometry.GetLocalSize().Y - Inset.Bottom
		};
	
		TOptional<double> Near = NearPlane.Get();
		TOptional<double> Far = FarPlane.Get();
		
		Positions.BarFarPlane = Positions.RelativeToBarPosition(Far.IsSet() ? Space->AlignedToRelativePosition(Far.GetValue()) : 1.0);
		Positions.BarNearPlane = Positions.RelativeToBarPosition(Near.IsSet() ? Space->AlignedToRelativePosition(Near.GetValue()) : 0.0f);
		
		return Positions;
	}
	
	return TOptional<FBarPositions>();
}

SDepthBar::FDepthSpace::FDepthSpace(const FVector& InOrigin, const FVector& InForward, const FBox& Bounds)
{
	Origin = InOrigin;
	Forward = InForward;
	
	const double BoundsMax = GetAlignedPosition(Bounds.Max);
	const double BoundsMin = GetAlignedPosition(Bounds.Min);
	
	if (BoundsMax > BoundsMin)
	{
		bFlipped = false;
		AlignedMax = BoundsMax;
		AlignedMin = BoundsMin;
	}
	else
	{
		bFlipped = true;
		AlignedMax = BoundsMin;
		AlignedMin = BoundsMax;
	}
}

double SDepthBar::FDepthSpace::GetAlignedPosition(const FVector& Position) const
{
	const FVector OriginToPosition = Position - Origin;
	const FVector ProjectedPosition = OriginToPosition.ProjectOnToNormal(Forward);
	return ProjectedPosition.Dot(Forward); 
}

double SDepthBar::FDepthSpace::AlignedToRelativePosition(double AlignedPosition) const
{
	const double Size = AlignedMax - AlignedMin;
	return Size != 0.0 ? ((AlignedPosition - AlignedMin) / Size) : 0.0;
}

double SDepthBar::FDepthSpace::RelativeToAlignedPosition(double RelativePosition) const
{
	return FMath::Lerp(AlignedMin, AlignedMax, RelativePosition);
}

double SDepthBar::FDepthSpace::RelativeToBoundsPosition(double RelativePosition) const
{
	const double Adjust = Origin.ProjectOnToNormal(Forward).Dot(Forward);
	if (bFlipped)
	{
		return FMath::Lerp(AlignedMax, AlignedMin, RelativePosition) + Adjust;
	}
	return FMath::Lerp(AlignedMin, AlignedMax, RelativePosition) + Adjust;
}

double SDepthBar::FDepthSpace::BoundsToRelativePosition(double BoundsPosition) const
{
	const double Adjusted = BoundsPosition - Origin.ProjectOnToNormal(Forward).Dot(Forward);
	if (bFlipped)
	{
		const double Size = AlignedMin - AlignedMax;
		return Size != 0.0 ? ((Adjusted - AlignedMax) / Size) : 0.0;
	}
	return AlignedToRelativePosition(Adjusted);
}

double SDepthBar::FBarPositions::WorldToBarPosition(const FVector& WorldPosition) const
{
	const double AxisPosition = DepthSpace.GetAlignedPosition(WorldPosition);
	const double RelativePosition = DepthSpace.AlignedToRelativePosition(AxisPosition);
	return RelativeToBarPosition(RelativePosition);
}

double SDepthBar::FBarPositions::RelativeToBarPosition(double RelativePosition) const
{
	return FMath::Clamp(1.0 - RelativePosition, 0.0, 1.0) * (BarMax - BarMin) + BarMin;
}

double SDepthBar::FBarPositions::BarToAlignedPosition(double BarPosition) const
{
	const double BarSize = BarMax - BarMin;
	const double RelativePosition = BarSize > 0.0 ? (1.0 - (BarPosition - BarMin) / BarSize) : 0.0;
	return DepthSpace.RelativeToAlignedPosition(RelativePosition);
}

EVisibility SDepthBar::GetFarPlaneLabelVisibility() const
{
	return HoverTarget == EDragTarget::Far_Handle || HoverTarget == EDragTarget::Slice ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SDepthBar::GetNearPlaneLabelVisibility() const
{
	return HoverTarget == EDragTarget::Near_Handle || HoverTarget == EDragTarget::Slice ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SDepthBar::GetFarPlaneText() const
{
	if (Mode.Get() == EMode::Orthographic)
	{
		TOptional<FDepthSpace> OptionalDepthSpace = DepthSpace.Get();
		if (const FDepthSpace* Space = OptionalDepthSpace.GetPtrOrNull())
		{
			TOptional<double> OptionalFarPlane = FarPlane.Get(); 
			if (const double* Far = OptionalFarPlane.GetPtrOrNull())
			{
				const double RelativePosition = Space->AlignedToRelativePosition(*Far);
				const double WorldPosition = Space->RelativeToBoundsPosition(RelativePosition);
			
				return FText::Format(LOCTEXT("OrthoFarPlaneFormat", "Far Plane: {0}"), UE::EditorWidgets::Private::GetDistanceDisplayText(WorldPosition));
			}
			return LOCTEXT("OrthoFarPlaneDefault", "Infinite Far Plane");
		}
	}
	return FText::GetEmpty();
}

FText SDepthBar::GetNearPlaneText() const
{
	if (Mode.Get() == EMode::Orthographic)
	{
		TOptional<FDepthSpace> OptionalDepthSpace = DepthSpace.Get();
		if (const FDepthSpace* Space = OptionalDepthSpace.GetPtrOrNull())
		{
			TOptional<double> OptionalNearPlane = NearPlane.Get(); 
			if (const double* Near = OptionalNearPlane.GetPtrOrNull())
			{
				const double RelativePosition = Space->AlignedToRelativePosition(*Near);
				const double WorldPosition = Space->RelativeToBoundsPosition(RelativePosition);

				return FText::Format(LOCTEXT("OrthoNearPlaneFormat", "Near Plane: {0}"), UE::EditorWidgets::Private::GetDistanceDisplayText(WorldPosition));
			}
			return LOCTEXT("OrthoNearPlaneDefault", "Infinite Near Plane");
		}
	}
	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
