// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShared.h"
#include "Math/Color.h"
#include "SceneManagement.h"
#include "Tools/Modes.h"

EDITORFRAMEWORK_API DECLARE_LOG_CATEGORY_EXTERN(LogEditorModes, Log, All);

// Builtin editor mode constants
namespace FBuiltinEditorModes
{
	/** Gameplay, editor disabled. */
	EDITORFRAMEWORK_API extern const FEditorModeID EM_None;

	/** Camera movement, actor placement. */
	EDITORFRAMEWORK_API extern const FEditorModeID EM_Default;

	/** Placement mode */
	EDITORFRAMEWORK_API extern const FEditorModeID EM_Placement;

	/** Mesh paint tool */
	EDITORFRAMEWORK_API extern const FEditorModeID EM_MeshPaint;

	/** Landscape editing */
	EDITORFRAMEWORK_API extern const FEditorModeID EM_Landscape;

	/** Foliage painting */
	EDITORFRAMEWORK_API extern const FEditorModeID EM_Foliage;

	/** Level editing mode */
	EDITORFRAMEWORK_API extern const FEditorModeID EM_Level;

	/** Streaming level editing mode */
	EDITORFRAMEWORK_API extern const FEditorModeID EM_StreamingLevel;

	/** Physics manipulation mode ( available only when simulating in viewport )*/
	EDITORFRAMEWORK_API extern const FEditorModeID EM_Physics;

	/** Actor picker mode, used to interactively pick actors in the viewport */
	EDITORFRAMEWORK_API extern const FEditorModeID EM_ActorPicker;

	/** Actor picker mode, used to interactively pick actors in the viewport */
	EDITORFRAMEWORK_API extern const FEditorModeID EM_SceneDepthPicker;
};

/** Material proxy wrapper that can be created on the game thread and passed on to the render thread. */
class FDynamicColoredMaterialRenderProxy : public FDynamicPrimitiveResource, public FColoredMaterialRenderProxy
{
public:
	/** Initialization constructor. */
	FDynamicColoredMaterialRenderProxy(const FMaterialRenderProxy* InParent,const FLinearColor& InColor)
	:	FColoredMaterialRenderProxy(InParent,InColor)
	{
	}
	virtual ~FDynamicColoredMaterialRenderProxy() = default;

	// FDynamicPrimitiveResource interface.
	virtual void InitPrimitiveResource(FRHICommandListBase& RHICmdList)
	{
	}
	virtual void ReleasePrimitiveResource()
	{
		delete this;
	}
};

