// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/SWidget.h"

class SWidget;


class IEditorViewportLayoutEntity : public TSharedFromThis<IEditorViewportLayoutEntity>
{
public:
	/** Virtual destruction */
	virtual ~IEditorViewportLayoutEntity() = default;

	/** Return a widget that is represents this entity */
	virtual TSharedRef<SWidget> AsWidget() const = 0;

	/** Set keyboard focus to this viewport entity */
	virtual void SetKeyboardFocus() = 0;

	/** Called when the parent layout is being destroyed */
	virtual void OnLayoutDestroyed() = 0;

	/** Called to save this item's settings in the specified config section */
	virtual void SaveConfig(const FString& ConfigSection) = 0;

	/** Get the type of this viewport as a name */
	virtual FName GetType() const = 0;

	/** Take a high res screen shot of viewport entity */
	virtual void TakeHighResScreenShot() const = 0;
};

class FEditorViewportLayout 
{
public:
	/** Virtual destruction */
	virtual ~FEditorViewportLayout() = default;

	virtual bool IsVisible() const { return true; }

	/**
	* @return All the viewports in this configuration
	*/
	const TMap< FName, TSharedPtr<IEditorViewportLayoutEntity> >& GetViewports() const { return Viewports; }

	virtual void FactoryPaneConfigurationFromTypeName(const FName& InLayoutConfigTypeName) = 0;
	virtual const FName GetActivePaneConfigurationTypeName() const = 0;

	virtual void SaveConfig(const FString& LayoutSring) const {}
	virtual void LoadConfig(const FString& LayoutString) {}

protected:
	/** List of all of the viewports in this layout, keyed on their config key */
	TMap< FName, TSharedPtr< IEditorViewportLayoutEntity > > Viewports;
};
