// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IToolkitHost;

/**
 * Mixin for asset editor customizations that provides access to the toolkit host of the asset editor.
 */
class FTypedElementAssetEditorToolkitHostMixin
{
public:
	/**
	 * Get the toolkit host associated with our asset editor.
	 */
	const IToolkitHost* GetToolkitHost() const
	{
		return ToolkitHost;
	}

	/**
	 * Get the toolkit host associated with our asset editor.
	 */
	IToolkitHost* GetMutableToolkitHost()
	{
		return ToolkitHost;
	}

	/**
	 * Set the toolkit host associated with our asset editor.
	 */
	void SetToolkitHost(IToolkitHost* InToolkitHost)
	{
		ToolkitHost = InToolkitHost;
	}

private:
	/** The toolkit host associated with our asset editor. */
	IToolkitHost* ToolkitHost = nullptr;
};
