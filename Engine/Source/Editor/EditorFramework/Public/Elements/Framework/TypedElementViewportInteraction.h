// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Elements/Framework/TypedElementAssetEditorToolkitHostMixin.h"
#include "Elements/Framework/TypedElementInterfaceCustomization.h"
#include "Elements/Framework/TypedElementListFwd.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "InputState.h"
#include "Math/Axis.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UnrealWidgetFwd.h"

#include "TypedElementViewportInteraction.generated.h"

#define UE_API EDITORFRAMEWORK_API

class UTypedElementSelectionSet;
struct FInputDeviceState;
struct FTypedElementHandle;

enum class ETypedElementViewportInteractionGizmoManipulationType : uint8
{
	/** The gizmo manipulation resulted in a drag operation */
	Drag,
	/** The gizmo manipulation resulted in a click operation */
	Click,
};

/**
 * Customization used to allow asset editors (such as the level editor) to override the base behavior of viewport interaction.
 */
class FTypedElementViewportInteractionCustomization : public FTypedElementAssetEditorToolkitHostMixin
{
public:
	virtual ~FTypedElementViewportInteractionCustomization() = default;

	//~ See UTypedElementViewportInteraction for API docs
	UE_API virtual bool GetGizmoPivotLocation(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, FVector& OutPivotLocation);
	UE_API virtual void PreGizmoManipulationStarted(TArrayView<const FTypedElementHandle> InElementHandles, const UE::Widget::EWidgetMode InWidgetMode);
	UE_API virtual void GizmoManipulationStarted(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode);
	UE_API virtual void GizmoManipulationDeltaUpdate(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, const EAxisList::Type InDragAxis, const FInputDeviceState& InInputState, const FTransform& InDeltaTransform, const FVector& InPivotLocation);
	UE_API virtual void GizmoManipulationStopped(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, const ETypedElementViewportInteractionGizmoManipulationType InManipulationType);
	UE_API virtual void PostGizmoManipulationStopped(TArrayView<const FTypedElementHandle> InElementHandles, const UE::Widget::EWidgetMode InWidgetMode);
	UE_API virtual void MirrorElement(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const FVector& InMirrorScale, const FVector& InPivotLocation);
	UE_API virtual bool GetFocusBounds(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, FBoxSphereBounds& OutBounds);
	UE_API virtual bool GetScaleForGizmo(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, FVector& OutScale);
};

/**
 * Utility to hold a typed element handle and its associated world interface and  viewport interaction customization.
 */
struct FTypedElementViewportInteractionElement
{
public:
	FTypedElementViewportInteractionElement() = default;

	FTypedElementViewportInteractionElement(TTypedElement<ITypedElementWorldInterface> InElementWorldHandle, FTypedElementViewportInteractionCustomization* InViewportInteractionCustomization)
		: ElementWorldHandle(MoveTemp(InElementWorldHandle))
		, ViewportInteractionCustomization(InViewportInteractionCustomization)
	{
	}

	FTypedElementViewportInteractionElement(const FTypedElementViewportInteractionElement&) = default;
	FTypedElementViewportInteractionElement& operator=(const FTypedElementViewportInteractionElement&) = default;

	FTypedElementViewportInteractionElement(FTypedElementViewportInteractionElement&&) = default;
	FTypedElementViewportInteractionElement& operator=(FTypedElementViewportInteractionElement&&) = default;

	FORCEINLINE explicit operator bool() const
	{
		return IsSet();
	}

	FORCEINLINE bool IsSet() const
	{
		return ElementWorldHandle.IsSet()
			&& ViewportInteractionCustomization;
	}

	//~ See UTypedElementViewportInteraction for API docs
	bool GetGizmoPivotLocation(const UE::Widget::EWidgetMode InWidgetMode, FVector& OutPivotLocation) const { return ViewportInteractionCustomization->GetGizmoPivotLocation(ElementWorldHandle, InWidgetMode, OutPivotLocation); }
	void GizmoManipulationStarted(const UE::Widget::EWidgetMode InWidgetMode) const { ViewportInteractionCustomization->GizmoManipulationStarted(ElementWorldHandle, InWidgetMode); }
	void GizmoManipulationDeltaUpdate(const UE::Widget::EWidgetMode InWidgetMode, const EAxisList::Type InDragAxis, const FInputDeviceState& InInputState, const FTransform& InDeltaTransform, const FVector& InPivotLocation) const { ViewportInteractionCustomization->GizmoManipulationDeltaUpdate(ElementWorldHandle, InWidgetMode, InDragAxis, InInputState, InDeltaTransform, InPivotLocation); }
	void GizmoManipulationStopped(const UE::Widget::EWidgetMode InWidgetMode, const ETypedElementViewportInteractionGizmoManipulationType InManipulationType) const { ViewportInteractionCustomization->GizmoManipulationStopped(ElementWorldHandle, InWidgetMode, InManipulationType); }
	void MirrorElement(const FVector& InMirrorScale, const FVector& InPivotLocation) const { ViewportInteractionCustomization->MirrorElement(ElementWorldHandle, InMirrorScale, InPivotLocation); }
	bool GetFocusBounds(FBoxSphereBounds& OutBounds) const { return ViewportInteractionCustomization->GetFocusBounds(ElementWorldHandle, OutBounds); }
	bool GetScaleForGizmo(FVector& OutScale) const { return ViewportInteractionCustomization->GetScaleForGizmo(ElementWorldHandle, OutScale); }

private:
	TTypedElement<ITypedElementWorldInterface> ElementWorldHandle;
	FTypedElementViewportInteractionCustomization* ViewportInteractionCustomization = nullptr;
};

/**
 * A utility to handle higher-level viewport interactions, but default via UTypedElementWorldInterface,
 * but asset editors can customize this behavior via FTypedElementViewportInteractionCustomization.
 */
UCLASS(MinimalAPI, Transient)
class UTypedElementViewportInteraction : public UObject, public TTypedElementInterfaceCustomizationRegistry<FTypedElementViewportInteractionCustomization>
{
	GENERATED_BODY()

public:
	/**
	 * Notify that the gizmo is potentially about to start manipulating the transform of the given set of elements.
	 * @note This list should have been pre-normalized via UTypedElementSelectionSet::GetNormalizedSelection or UTypedElementSelectionSet::GetNormalizedElementList, or FLevelEditorViewportClient::GetElementsToManipulate.
	 */
	UE_API void BeginGizmoManipulation(FTypedElementListConstRef InElementsToMove, const UE::Widget::EWidgetMode InWidgetMode);

	/**
	 * Notify that the gizmo has manipulated the transform of the given set of elements.
	 * @note This list should have been pre-normalized via UTypedElementSelectionSet::GetNormalizedSelection or UTypedElementSelectionSet::GetNormalizedElementList, or FLevelEditorViewportClient::GetElementsToManipulate.
	 */
	UE_API void UpdateGizmoManipulation(FTypedElementListConstRef InElementsToMove, const UE::Widget::EWidgetMode InWidgetMode, const EAxisList::Type InDragAxis, const FInputDeviceState& InInputState, const FTransform& InDeltaTransform);
	
	/**
	 * Notify that the gizmo has finished manipulating the transform of the given set of elements.
	 * @note This list should have been pre-normalized via UTypedElementSelectionSet::GetNormalizedSelection or UTypedElementSelectionSet::GetNormalizedElementList, or FLevelEditorViewportClient::GetElementsToManipulate.
	 */
	UE_API void EndGizmoManipulation(FTypedElementListConstRef InElementsToMove, const UE::Widget::EWidgetMode InWidgetMode, const ETypedElementViewportInteractionGizmoManipulationType InManipulationType);

	/**
	 * Apply the given delta to the specified element.
	 * This performs a similar operation to an in-flight gizmo manipulation, but without any pre/post-change notification.
	 */
	UE_API void ApplyDeltaToElement(const FTypedElementHandle& InElementHandle, const UE::Widget::EWidgetMode InWidgetMode, const EAxisList::Type InDragAxis, const FInputDeviceState& InInputState, const FTransform& InDeltaTransform);

	/**
	 * Apply the given mirror scale to the specified element.
	 */
	UE_API void MirrorElement(const FTypedElementHandle& InElementHandle, const FVector& InMirrorScale);

	/**
	 *	Calculate the bounds of all elements in the list, to be focused on in the viewport
	 *	@note This list should have been pre-normalized via UTypedElementSelectionSet::GetNormalizedSelection or UTypedElementSelectionSet::GetNormalizedElementList, or FLevelEditorViewportClient::GetElementsToManipulate.
	 */
	UE_API bool GetFocusBounds(FTypedElementListConstRef InElements, FBoxSphereBounds& OutBox);

	/**
	 *	Calculate the combined, signed scale of all elements in the list, to be used for scale delta modifications (before applying). If there are multiple objects of differing scale, return the total bounds.
	 *	@note This list should have been pre-normalized via UTypedElementSelectionSet::GetNormalizedSelection or UTypedElementSelectionSet::GetNormalizedElementList, or FLevelEditorViewportClient::GetElementsToManipulate.
	 */
	UE_API bool GetScaleForGizmo(FTypedElementListConstRef InElements, FVector& OutScale);
private:
	/**
	 * Attempt to resolve the selection interface and viewport interaction customization for the given element, if any.
	 */
	UE_API FTypedElementViewportInteractionElement ResolveViewportInteractionElement(const FTypedElementHandle& InElementHandle) const;
};

#undef UE_API
