// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementHandle.h"
#include "UObject/Interface.h"

#include "TypedElementDetailsInterface.generated.h"

/**
 * Proxy instance to provide a UObject for editing by a details panel.
 * This instance will exist as long as the details panel is using it, so gives a lifetime to potentially 
 * synthesized UObject instances that are created purely for editing (eg, on instances).
 */
class ITypedElementDetailsObject
{
public:
	virtual ~ITypedElementDetailsObject() = default;

	/**
	 * Get the underlying UObject that should be edited by the details panel, if any.
	 */
	virtual UObject* GetObject() = 0;

	/**
	 * Called during GC to collect references.
	 */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) {}
};

UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UTypedElementDetailsInterface : public UInterface
{
	GENERATED_BODY()
};

class ITypedElementDetailsInterface
{
	GENERATED_BODY()

public:
	/**
	 * Is the given element considered "top-level" for editing?
	 * ie) Should it be edited simply from being part of the main selection set for a level?
	 */
	virtual bool IsTopLevelElement(const FTypedElementHandle& InElementHandle) { return true; }

	/**
	 * Get the proxy instance for the given element, if any.
	 * @see ITypedElementDetailsObject for more details about this proxy instance.
	 */
	virtual TUniquePtr<ITypedElementDetailsObject> GetDetailsObject(const FTypedElementHandle& InElementHandle) { return nullptr; }
};

template <>
struct TTypedElement<ITypedElementDetailsInterface> : public TTypedElementBase<ITypedElementDetailsInterface>
{
	bool IsTopLevelElement() const { return InterfacePtr->IsTopLevelElement(*this); }
	TUniquePtr<ITypedElementDetailsObject> GetDetailsObject() const { return InterfacePtr->GetDetailsObject(*this); }
};
