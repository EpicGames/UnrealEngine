// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentVisualizer.h" // FCachedComponentVisualizer
#include "Elements/Framework/TypedElementHandle.h"
#include "UObject/Interface.h"

#include "TypedElementEditorVisualizerInterface.generated.h"

class UTypedElementSelectionSet;

UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UTypedElementEditorVisualizerInterface : public UInterface
{
	GENERATED_BODY()
};

UE_EXPERIMENTAL(5.8, "ITypedElementEditorVisualizerInterface is currently used to bridge non-actor "
	"elements to actor component visualizers, but it may be removed as another visualizer system comes online.")
class ITypedElementEditorVisualizerInterface
{
	GENERATED_BODY()

public:

	//~ Mirrors UUnrealEdEngine::FComponentVisualizerForSelection to avoid exposing that struct, as this interface might
	//~  be temporary.
	struct FComponentVisualizerForSelection
	{
		FCachedComponentVisualizer ComponentVisualizer;
		TOptional<TFunction<bool(void)>> IsEnabledDelegate;
	};

	/**
	 * For given element, appends any additional visualizers to use
	 */
	virtual bool GetComponentVisualizersToUse(const FTypedElementHandle& InElementHandle, const UTypedElementSelectionSet* SelectionSet, 
		TArray<FComponentVisualizerForSelection>& VisualizersOut) const
	{ 
		return false; 
	}
};

template <>
struct TTypedElement<ITypedElementEditorVisualizerInterface> : public TTypedElementBase<ITypedElementEditorVisualizerInterface>
{
	bool GetComponentVisualizersToUse(const UTypedElementSelectionSet* SelectionSet,
		TArray<ITypedElementEditorVisualizerInterface::FComponentVisualizerForSelection>& VisualizersOut) const
	{ 
		return InterfacePtr->GetComponentVisualizersToUse(*this, SelectionSet, VisualizersOut);
	}
};
