// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Instances/InstancedPlacementClientInfo.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "AssetFactoryInterface.generated.h"

class IAssetFactoryInterface;
class UInstancedPlacemenClientSettings;
class UObject;
struct FAssetPlacementInfo;
struct FPlacementOptions;
struct FTypedElementHandle;

UINTERFACE(MinimalAPI)
class UAssetFactoryInterface : public UInterface
{
	GENERATED_BODY()
};

class IAssetFactoryInterface
{
	GENERATED_BODY()

public:
	/**
	 * Given an FAssetData, determine if this asset factory can place any elements.
	 *
	 * @returns true if the factory can be used to place elements.
	 */
	virtual bool CanPlaceElementsFromAssetData(const FAssetData& InAssetData) = 0;

	/**
	 * Performs any final tweaking of the PlacementInfo that the asset factory may need to do. This includes final adjustments to things like transforms.
	 * This should NOT perform any viewport or editor specific adjustments, such as grid snapping, alignment to hit objects, or undo tracking.
	 *
	 * @returns true if the asset is still valid to place after final adjustments.
	 */
	virtual bool PrePlaceAsset(FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions) = 0;

	/**
	 * Places the asset.
	 *
	 * @returns valid FTypedElementHandles that were placed by the factory from the given asset.
	 */
	virtual TArray<FTypedElementHandle> PlaceAsset(const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions) = 0;

	/**
	 * Finalizes any placed elements based on adjustments the factory may need to do.
	 * This should NOT include any adjustments from viewport or asset editor specific functionality, such as finalizing undo tracking.
	 */
	virtual void PostPlaceAsset(TArrayView<const FTypedElementHandle> InHandle, const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions) = 0;

	/**
	 * Given an FTypedElementHandle, attempt to resolve the handle to the FAssetData which may have placed it.
	 * The FAssetData may be a wrapped type, like a static mesh component component inside a static mesh actor.
	 *
	 * @returns the FAssetData which corresponds to the placed FTypedElementHandle. The returned FAssetData may be invalid.
	 */
	virtual FAssetData GetAssetDataFromElementHandle(const FTypedElementHandle& InHandle) = 0;

	/**
	 * Sets up any states that the factory needs to track across a placement call.
	 * Called once per factory during a PlaceAssets call from the placement subsystem.
	 * 
	 * For example, for instanced static mesh placement, we may want to disable rebuilding the parent component's tree until EndPlacement is called.
	 */
	virtual void BeginPlacement(const FPlacementOptions& InPlacementOptions) = 0;

	/**
	 * Tears down any state, or kicks off any rebuilds of data that this asset type may need.
	 * Called once per factory during a PlaceAssets call from the placement subsystem.
	 *
	 * For example, for instanced static mesh placement, we'd rebuild the parent component's tree here.
	 */
	 virtual void EndPlacement(TArrayView<const FTypedElementHandle> InPlacedElements, const FPlacementOptions& InPlacementOptions) = 0;

	 /**
	  * Returns the settings object which this factory will use to customize placement settings, based on the given placement information.
	  */
	 virtual UInstancedPlacemenClientSettings* FactorySettingsObjectForPlacement(const FAssetData& InAssetData, const FPlacementOptions& InPlacementOptions) = 0;
};
