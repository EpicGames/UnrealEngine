// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"

#include "DepthBarStyle.generated.h"

struct UE_EXPERIMENTAL(5.7, "Represents the style of an SDepthBar") FDepthBarStyle;
USTRUCT()
struct FDepthBarStyle : public FSlateWidgetStyle
{
	GENERATED_BODY()
	
	EDITORFRAMEWORK_API FDepthBarStyle();
	EDITORFRAMEWORK_API FDepthBarStyle(const FDepthBarStyle&);
	EDITORFRAMEWORK_API virtual ~FDepthBarStyle() override;
	
	EDITORFRAMEWORK_API virtual void GetResources(TArray<const FSlateBrush*>& OutBrushes) const override;

	EDITORFRAMEWORK_API static const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };
	
	EDITORFRAMEWORK_API static const FDepthBarStyle& GetDefault();
	
	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin Padding = FMargin(4.0f);
	FDepthBarStyle& SetPadding(const FMargin& InPadding) { Padding = InPadding; return *this; };
	
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush BackgroundBrush;
	FDepthBarStyle& SetBackgroundBrush(const FSlateBrush& InBackground) { BackgroundBrush = InBackground; return *this; };
	
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush TrackBrush;
	FDepthBarStyle& SetTrack(const FSlateBrush& InTrack) { TrackBrush = InTrack; return *this; };
	
	UPROPERTY(EditAnywhere, Category = Appearance)
	float TrackWidth = 2.0f;
	FDepthBarStyle& SetTrackWidth(float InTrackWidth) { TrackWidth = InTrackWidth; return *this; };

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush SliceNormalBrush;
	FDepthBarStyle& SetSliceNormal(const FSlateBrush& InSliceNormal) { SliceNormalBrush = InSliceNormal; return *this; };
	
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush SliceHoveredBrush;
	FDepthBarStyle& SetSliceHovered(const FSlateBrush& InSliceHovered) { SliceHoveredBrush = InSliceHovered; return *this; };
	
	UPROPERTY(EditAnywhere, Category = Appearance)
	float SliceWidth = 16.0f;
	FDepthBarStyle& SetSliceWidth(float InSliceWidth) { SliceWidth = InSliceWidth; return *this; };
	
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush SliceTopBrush;
	FDepthBarStyle& SetSliceTopBrush(const FSlateBrush& InSliceTopBrush) { SliceTopBrush = InSliceTopBrush; return *this; };
	
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush SliceTopHoveredBrush;
	FDepthBarStyle& SetSliceTopHovered(const FSlateBrush& InSliceTopHovered) { SliceTopHoveredBrush = InSliceTopHovered; return *this; };
	
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush SliceBottomBrush;
	FDepthBarStyle& SetSliceBottom(const FSlateBrush& InSliceBottom) { SliceBottomBrush = InSliceBottom; return *this; };
	
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush SliceBottomHoveredBrush;
	FDepthBarStyle& SetSliceBottomHovered(const FSlateBrush& InSliceBottomHovered) { SliceBottomHoveredBrush = InSliceBottomHovered; return *this; };
	
	UPROPERTY(EditAnywhere, Category = Appearance)
	FVector2f SliceHandleSize = FVector2f(16.0f, 16.0f);
	FDepthBarStyle& SetSliceHandleSize(const FVector2f& InHandleSize) { SliceHandleSize = InHandleSize; return *this; };
	
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush FarPlaneButtonBrush;
	FDepthBarStyle& SetFarPlaneButtonBrush(const FSlateBrush& InFarPlaneButton) { FarPlaneButtonBrush = InFarPlaneButton; return *this; };
	
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush NearPlaneButtonBrush;
	FDepthBarStyle& SetNearPlaneButtonBrush(const FSlateBrush& InNearPlaneButton) { NearPlaneButtonBrush = InNearPlaneButton; return *this; };
	
	UPROPERTY(EditAnywhere, Category = Appearance)
	FVector2f PlaneButtonSize = FVector2f(16.0f, 16.0f);
	FDepthBarStyle& SetPlaneButtonSize(const FVector2f& InHandleSize) { PlaneButtonSize = InHandleSize; return *this; };
	
	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin FarPlaneButtonPadding = FMargin(4.0f);
	FDepthBarStyle& SetFarPlaneButtonPadding(const FMargin& InPlaneButtonPadding) { FarPlaneButtonPadding = InPlaneButtonPadding; return *this; };
	
	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin NearPlaneButtonPadding = FMargin(4.0f);
	FDepthBarStyle& SetNearPlaneButtonPadding(const FMargin& InPlaneButtonPadding) { NearPlaneButtonPadding = InPlaneButtonPadding; return *this; };
	
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateColor PlaneButtonNormalColor = EStyleColor::ForegroundHeader;
	FDepthBarStyle& SetPlaneButtonNormalColor(const FSlateColor& InColor) { PlaneButtonNormalColor = InColor; return *this; };
	
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateColor PlaneButtonHoveredColor = EStyleColor::ForegroundHover;
	FDepthBarStyle& SetPlaneButtonHoveredColor(const FSlateColor& InColor) { PlaneButtonHoveredColor = InColor; return *this; };
	
};
