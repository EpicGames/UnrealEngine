// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Elements/Framework/TypedElementListFwd.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "Math/Transform.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UnrealWidgetFwd.h"

#include "EditorElementSubsystem.generated.h"

#define UE_API EDITORFRAMEWORK_API

class UObject;
struct FTypedElementHandle;

UCLASS(MinimalAPI, Transient)
class UEditorElementSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	/**
	 * Sets the world transform of the given element handle, if possible.
	 * @returns false if the world transform could not be set.
	 */
	static UE_API bool SetElementTransform(FTypedElementHandle InElementHandle, const FTransform& InWorldTransform);

	/**
	 * Return a normalized selection set like the editor would use for the gizmo manipulations
	 */
	static UE_API FTypedElementListRef GetEditorNormalizedSelectionSet(const UTypedElementSelectionSet& SelectionSet);

	/**
	 * Return only the manipulatable elements from a selection list.
	 * Require a editor normalized selection list to behave like the editor selection
	 * Note: A manipulable element is a element that would be move when manipulating the gizmo in the editor
	 */
	static UE_API FTypedElementListRef GetEditorManipulableElements(const FTypedElementListRef& NormalizedSelection, UE::Widget::EWidgetMode InManipulationType = UE::Widget::WM_Translate, UWorld* InRequiredWorld = nullptr);

	/**
	 * Return the most recently selected element that is manipulable.
	 * Require a editor normalized selection list to behave like the editor selection
	 * Note: A manipulable element is a element that would be move when manipulating the gizmo in the editor.
	 */
	static UE_API TTypedElement<ITypedElementWorldInterface> GetLastSelectedEditorManipulableElement(const FTypedElementListRef& NormalizedSelection, UE::Widget::EWidgetMode InManipulationType = UE::Widget::WM_Translate, UWorld* InRequiredWorld = nullptr);

	static UE_API bool IsElementEditorManipulable(const TTypedElement<ITypedElementWorldInterface>& WorldElement, UE::Widget::EWidgetMode InManipulationType = UE::Widget::WM_Translate, UWorld* InRequiredWorld = nullptr);
};

#undef UE_API
