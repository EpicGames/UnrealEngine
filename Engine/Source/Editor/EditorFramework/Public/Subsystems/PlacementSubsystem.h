// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Delegates/Delegate.h"
#include "EditorSubsystem.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Math/Transform.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ScriptInterface.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "PlacementSubsystem.generated.h"

#define UE_API EDITORFRAMEWORK_API

class FSubsystemCollectionBase;
class IAssetFactoryInterface;
class UAssetFactoryInterface;
class UClass;
class UInstancedPlacemenClientSettings;
class ULevel;
class UObject;
struct FTypedElementHandle;

USTRUCT()
struct FAssetPlacementInfo
{
	GENERATED_BODY()

	// The asset data which should be placed.
	UPROPERTY()
	FAssetData AssetToPlace;

	// If set, will override the name on placed elements instead of factory defined defaults.
	UPROPERTY()
	FName NameOverride;

	// If set, the factory will attempt to place inside the given level. World partitioning may ultimately override this preference.
	UPROPERTY()
	TWeakObjectPtr<ULevel> PreferredLevel;

	// The finalized transform where the factory should be place the asset. This should include any location snapping or other considerations from viewports or editor settings.
	UPROPERTY()
	FTransform FinalizedTransform;

	// If set, will use the given factory to place the asset, instead of allowing the placement subsystem to determine which factory to use.
	UPROPERTY()
	TScriptInterface<IAssetFactoryInterface> FactoryOverride;

	/**
	 * The Guid which corresponds to the item that should be placed.
	 * If unset, the asset package's persistent guid will be used.
	 * Factories should use this to tie any decomposed assets together. For example, the ItemGuid would correspond to the client in an AISMPartitionActor for tracking all static meshes which make up a decomposed actor.
	 */
	UPROPERTY()
	FGuid ItemGuid;

	UPROPERTY()
	TObjectPtr<UInstancedPlacemenClientSettings> SettingsObject = nullptr;
};

USTRUCT()
struct FPlacementOptions
{
	GENERATED_BODY()
	
	/**
	 * The guid to use by factories for instanced placement. If unset, factories will not use instanced placement.
	 * This is used to reduce contention within one file per actor within a partition.
	 */
	UPROPERTY()
	FGuid InstancedPlacementGridGuid;

	// If true, asset factory implementations should prefer a batch placement algorithm (like duplicating an object) over a single placement algorithm.
	UPROPERTY()
	bool bPreferBatchPlacement = true;

	// If true, creates transient preview elements, which are transient and not saved to a level.
	UPROPERTY()
	bool bIsCreatingPreviewElements = false;
};

UCLASS(MinimalAPI, Transient)
class UPlacementSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	// USubsystem Interface
	UE_API void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API void Deinitialize() override;
	// End USubsystem Interface

	/**
	 * Places a single asset based on the given FAssetPlacementInfo and FPlacementOptions.
	 * @returns an array of FTypedElementHandles corresponding to any successfully placed elements.
	 */
	UE_API TArray<FTypedElementHandle> PlaceAsset(const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions);

	/**
	 * Places multiple assets based on the given FAssetPlacementInfos and FPlacementOptions.
	 * @returns an array of FTypedElementHandles corresponding to any successfully placed elements.
	 */
	UE_API TArray<FTypedElementHandle> PlaceAssets(TArrayView<const FAssetPlacementInfo> InPlacementInfos, const FPlacementOptions& InPlacementOptions);

	/**
	 * Finds a registered AssetFactory for the given FAssetData.
	 * @returns the first found factory, or nullptr if no valid factory is found.
	 */
	UE_API TScriptInterface<IAssetFactoryInterface> FindAssetFactoryFromAssetData(const FAssetData& InAssetData);

	/**
	 * Determines if the placement system is currently placing preview elements.
	 * @returns true if the current PlaceAssets call is creating preview elements.
	 */
	UE_API bool IsCreatingPreviewElements() const;

	UE_API FSimpleMulticastDelegate& OnPlacementFactoriesRegistered();

	UE_API TScriptInterface<IAssetFactoryInterface> GetAssetFactoryFromFactoryClass(UClass* InFactoryInterfaceClass) const;

	UE_DEPRECATED(5.4, "This overload never worked as intended. Use the UClass* overload instead.")
	UE_API TScriptInterface<IAssetFactoryInterface> GetAssetFactoryFromFactoryClass(TSubclassOf<UClass> InFactoryInterfaceClass) const;
	
	UE_API void RegisterAssetFactory(TScriptInterface<IAssetFactoryInterface> AssetFactory);

	UE_API void UnregisterAssetFactory(TScriptInterface<IAssetFactoryInterface> AssetFactory);
	
private:
	UE_API void RegisterPlacementFactories();
	UE_API void UnregisterPlacementFactories();

	UPROPERTY()
	TArray<TScriptInterface<IAssetFactoryInterface>> AssetFactories;

	FSimpleMulticastDelegate PlacementFactoriesRegistered;

	bool bIsCreatingPreviewElements = false;
};

#undef UE_API
