// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "EditorSubsystem.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/TabManager.h"
#include "ILevelEditor.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/NotifyHook.h"
#include "StatusBarSubsystem.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "Toolkits/IToolkitHost.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "ToolMenuDelegates.h"

#include "AssetEditorModeUILayer.generated.h"

#define UE_API EDITORFRAMEWORK_API

class FExtender;
class FLayoutExtender;
class FWorkspaceItem;
class ILevelEditor;
class IToolkit;
class SBorder;
class SDockTab;
class UObject;
struct FSlateIcon;

UCLASS(MinimalAPI)
class UAssetEditorUISubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	static UE_API const FName VerticalToolbarID;
	static UE_API const FName TopLeftTabID;
	static UE_API const FName BottomLeftTabID;
	static UE_API const FName TopRightTabID;
	static UE_API const FName BottomRightTabID;

protected:
	virtual void RegisterLayoutExtensions(FLayoutExtender& Extender) {};
};

class FAssetEditorModeUILayer : public TSharedFromThis<FAssetEditorModeUILayer>
{
public:
	UE_API FAssetEditorModeUILayer(const IToolkitHost* InToolkitHost);
	FAssetEditorModeUILayer() {};
	virtual ~FAssetEditorModeUILayer() = default;
	/** Called by SLevelEditor to notify the toolbox about a new toolkit being hosted */
	UE_API virtual void OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit);

	/** Called by SLevelEditor to notify the toolbox about an existing toolkit no longer being hosted */
	UE_API virtual void OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit);
	UE_API virtual TSharedPtr<FTabManager> GetTabManager();
	UE_API virtual TSharedPtr<FWorkspaceItem> GetModeMenuCategory() const;
	UE_API virtual void SetModePanelInfo(const FName InTabSpawnerID, const FMinorTabConfig& InTabInfo);
	UE_API virtual TMap<FName, TWeakPtr<SDockTab>> GetSpawnedTabs();

	/** Called by the Toolkit Host to set the name of the ModeToolbar this mode will extend */
	UE_API virtual void SetSecondaryModeToolbarName(FName InName);
	
	virtual FSimpleDelegate& ToolkitHostReadyForUI()
	{
		return OnToolkitHostReadyForUI;
	};
	virtual FSimpleDelegate& ToolkitHostShutdownUI()
	{
		return OnToolkitHostShutdownUI;
	}
	virtual const FName GetStatusBarName() const
	{
		return ToolkitHost->GetStatusBarName();
	}
	virtual const FName GetSecondaryModeToolbarName() const
	{
		return SecondaryModeToolbarName;
	}
	virtual const TSharedRef<FUICommandList> GetModeCommands() const
	{
		return ModeCommands;
	}
	/* Called by the mode toolkit to extend the toolbar */
	virtual FNewToolMenuDelegate& RegisterSecondaryModeToolbarExtension()
	{
		return OnRegisterSecondaryModeToolbarExtension;
	}

protected:
	UE_API const FOnSpawnTab& GetStoredSpawner(const FName TabID);
	UE_API void RegisterModeTabSpawners();
	UE_API void RegisterModeTabSpawner(const FName TabID);
	UE_API TSharedRef<SDockTab> SpawnStoredTab(const FSpawnTabArgs& Args, const FName TabID);
	UE_API bool CanSpawnStoredTab(const FSpawnTabArgs& Args, const FName TabID);
	UE_API FText GetTabSpawnerName(const FName TabID) const;
	UE_API FText GetTabSpawnerTooltip(const FName TabID) const;
	UE_API const FSlateIcon& GetTabSpawnerIcon(const FName TabID) const;

protected:
	/** The host of the toolkits created by modes */
	const IToolkitHost* ToolkitHost;
	TArray<FName> ModeTabIDs;
	TWeakPtr<IToolkit> HostedToolkit;
	TMap<FName, FMinorTabConfig> RequestedTabInfo;
	TMap<FName, TWeakPtr<SDockTab>> SpawnedTabs;
	FSimpleDelegate OnToolkitHostReadyForUI;
	FSimpleDelegate OnToolkitHostShutdownUI;

	/* The name of the mode toolbar that this mode will extend (appears below the main toolbar) */
	FName SecondaryModeToolbarName;

	/* Delegate called to actually extend the mode toolbar */
	FNewToolMenuDelegate OnRegisterSecondaryModeToolbarExtension;

	/* A list of commands this ModeUILayer is aware of, currently only passed into the Mode Toolbar */
	TSharedRef<FUICommandList> ModeCommands;
};

#undef UE_API
