// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Framework/Docking/TabManager.h"

class FEdMode;
class UEdMode;
class IToolkitHost;
class FEditorModeTools;
class FAssetEditorModeUILayer;

/** Some toolkits can be spawned as either standalone tools or within an existing level editing UI */
namespace EToolkitMode
{
	enum Type
	{
		/** Stand-alone asset editing "app" */
		Standalone,

		/** World-centric asset editor, with an interface that sits alongside the level editor */
		WorldCentric,
	};
}


namespace EToolkitTabSpot
{
	enum Type
	{
		// A good place to put a details tab or property tree, or other tabs that display information about 
		// the current object the user has selected
		Details,

		// This is for document windows, such as a graph editor or a profiling results panel
		Document,

		// Put navigation-related panels here, such as a list of editable sub-objects
		Navigation,

		// Tool bar panels are usually displayed at the top of the tool in a narrow strip
		ToolBar,

		// Placement panels host tools for creating new things.  This is usually displayed in the top left or center left of a window
		Placement,

		// Preview viewports
		Viewport,

		// Area below the level editor. Useful for things that should fill the entire width of the level editor
		BelowLevelEditor,
	};
}


/**
 * Interface for editor toolkits (asset editors and mode tools)
 */
class IToolkit
{
public:

	virtual ~IToolkit() = default;

	/** Register tabs that this toolkit can spawn with the TabManager */
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) = 0;

	/** Unregister tabs that this toolkit can spawn */	
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) = 0;

public:

	/** Returns the invariant name of this toolkit type */
	virtual FName GetToolkitFName() const = 0;

	/** Returns the invariant name of this toolkit type, used for establishing the context for help, documentation and tutorials */
	virtual FName GetToolkitContextFName() const = 0;

	/** Returns the localized name of this toolkit type (typically just "<ClassName> editor") */
	virtual FText GetBaseToolkitName() const = 0;

	/** Returns the localized name of this toolkit */
	virtual FText GetToolkitName() const = 0;

	/** Returns the localized tab suffix */
	virtual FText GetTabSuffix() const = 0;

	/** Returns the localized tooltip text of this toolkit */
	virtual FText GetToolkitToolTipText() const = 0;

	/** Returns the workspace menu category of this toolkit */
	virtual TSharedRef<FWorkspaceItem> GetWorkspaceMenuCategory() const = 0;

	/** Returns the localize prefix string to use for tab labels in world-centric mode. */
	virtual FString GetWorldCentricTabPrefix() const = 0;

	/** Returns true if this toolkit is used to edit assets (even if it's not necessarily editing one right now.) */
	virtual bool IsAssetEditor() const = 0;

	/** For asset editor toolkits, returns the UObjects for the assets currently being edited */
	virtual const TArray< UObject* >* GetObjectsCurrentlyBeingEdited() const = 0;

	/** @return Returns the color and opacity to use for the color that appears behind the tab text for this toolkit's tab in world-centric mode. */
	virtual FLinearColor GetWorldCentricTabColorScale() const = 0;

	/** @return Returns true if this toolkit is currently hosted.  All toolkits are hosted except during a shutdown situation. */
	virtual bool IsHosted() const = 0;

	/** @return Returns the toolkit host for this toolkit */
	virtual const TSharedRef<IToolkitHost> GetToolkitHost() const = 0;

	/**
	 * Processes any UI commands which are activated by the specified event
	 *
	 * @param InKeyEvent	The event to check
	 *
	 * @return true if an action was processed
	 */
	virtual bool ProcessCommandBindings(const FKeyEvent& InKeyEvent) const = 0;

	/** Call this function to bring all of this toolkit's tabs to the foreground in their respective stacks.  Also causes the toolkit's host window to be foregrounded, too! */
	virtual void BringToolkitToFront() = 0;

	/** Returns if this is a IBlueprintEditor derivation */
	virtual bool IsBlueprintEditor() const = 0;

	/* Called when a toolkit has been brought to the 'front' */
	virtual void ToolkitBroughtToFront() {};

	/* Returns true if this is the default generic asset editor used by multiple classes */
	virtual bool IsSimpleAssetEditor() const { return false; }

	/* Gets the mode manager for this toolkit */
	virtual FEditorModeTools& GetEditorModeManager() const = 0;

	/** @returns the inline content that this toolkit returns */
	virtual TSharedPtr<SWidget> GetInlineContent() const = 0;

	virtual void SetModeUILayer(const TSharedPtr<FAssetEditorModeUILayer> InLayer) {};
};



