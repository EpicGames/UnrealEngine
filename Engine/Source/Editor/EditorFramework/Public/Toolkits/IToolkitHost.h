// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FEditorModeTools;
class UTypedElementCommonActions;
class IAssetViewport;

/**
 * Base interface class for toolkit hosts
 */
class IToolkitHost
{

public:
	virtual ~IToolkitHost() = default;

	/* Notificaton when the active viewport changed */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnActiveViewportChanged,  TSharedPtr<IAssetViewport>, TSharedPtr<IAssetViewport>)

	/** Gets a widget that can be used to parent a modal window or pop-up to.  You shouldn't be using this widget for
	    anything other than parenting, as the type of widget and behavior/lifespan is completely up to the host. */
	virtual TSharedRef< class SWidget > GetParentWidget() = 0;

	/** Brings this toolkit host's window (and tab, if it has one), to the front */
	virtual void BringToFront() = 0;

	/** Access the toolkit host's tab manager */
	virtual TSharedPtr< class FTabManager > GetTabManager() const = 0;

	/** Called when a toolkit is opened within this host */
	virtual void OnToolkitHostingStarted( const TSharedRef< class IToolkit >& Toolkit ) = 0;

	/** Called when a toolkit is no longer being hosted within this host */
	virtual void OnToolkitHostingFinished( const TSharedRef< class IToolkit >& Toolkit ) = 0;

	/** @return For world-centric toolkit hosts, gets the UWorld associated with this host */
	virtual class UWorld* GetWorld() const = 0;

	/** Returns the mode manager for this toolkit host. For standalone toolkits */
	virtual FEditorModeTools& GetEditorModeManager() const = 0;

	/** Returns the common actions implementation for this toolkit host */
	virtual UTypedElementCommonActions* GetCommonActions() const = 0;

	/** Returns the status bar name that can be used with UStatusBarSubsystem to display messages 
	    in the status bar of this toolkit host. Should return NAME_None if not applicable. */
	virtual FName GetStatusBarName() const = 0;

	/** Allows Toolkits to push widgets to the viewport.  Passing a nullptr for the Viewport will add  or 
	    remove the OverlayWidget to or from the ActiveViewport */
	virtual void AddViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, TSharedPtr<IAssetViewport> InViewport = nullptr)
	{
		AddViewportOverlayWidget(InOverlaidWidget, INDEX_NONE, InViewport);		
	}

	/** Allows Toolkits to push widgets to the viewport with a specific Z-Order.  Passing a nullptr for the Viewport will add  or 
		remove the OverlayWidget to or from the ActiveViewport */
	virtual void AddViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder, TSharedPtr<IAssetViewport> InViewport = nullptr)  {}

	virtual void RemoveViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, TSharedPtr<IAssetViewport> InViewport = nullptr) {}

	/** Return the size of the active viewport, accounting for DPI Scale */
	UE_DEPRECATED(5.6, "Viewport sizes are represented in float, please update your callsites to use GetActiveViewportWidgetSize and use FVector2f for slate sizes.")
	virtual FVector2D GetActiveViewportSize() final
	{
		return FVector2D(GetActiveViewportWidgetSize());
	}

	/** Return the size of the active viewport, accounting for DPI Scale */
	virtual FVector2f GetActiveViewportWidgetSize() const
	{
		return FVector2f::ZeroVector;
	}

	/** Gets a multicast delegate which is executed whenever the toolkit host's active viewport changes. */
	virtual FOnActiveViewportChanged& OnActiveViewportChanged() = 0;

};
