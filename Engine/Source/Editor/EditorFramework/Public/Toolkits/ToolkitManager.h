// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"

class IToolkit;
class IToolkitHost;
class UObject;

/**
 * Singleton that managers instances of editor toolkits
 */
class FToolkitManager
{

public:
	/** Get the singleton instance of the asset editor manager */
	EDITORFRAMEWORK_API static FToolkitManager& Get();

	/** Call this to register a newly-created toolkit. */
	EDITORFRAMEWORK_API void RegisterNewToolkit( const TSharedRef< IToolkit > NewToolkit );

	/** Call this to close an existing toolkit */
	EDITORFRAMEWORK_API void CloseToolkit( const TSharedRef< IToolkit > ClosingToolkit );

	/** Called by a toolkit host itself right before it goes away, so that we can make sure the toolkits are destroyed too */
	EDITORFRAMEWORK_API void OnToolkitHostDestroyed( IToolkitHost* HostBeingDestroyed );

	/** Tries to find an open asset editor that's editing the specified asset, and returns the toolkit for that asset editor */
	EDITORFRAMEWORK_API TSharedPtr< IToolkit > FindEditorForAsset( const UObject* Asset );

private:
	// Buried constructor since the asset editor manager is a singleton
	FToolkitManager();


private:

	/** List of all currently open toolkits */
	TArray< TSharedPtr< IToolkit > > Toolkits;
};



