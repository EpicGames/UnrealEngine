// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "AssetEditorContextInterface.generated.h"

class IToolkitHost;
class UTypedElementCommonActions;
class UTypedElementSelectionSet;

UINTERFACE(Blueprintable, MinimalAPI)
class UAssetEditorContextInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * An interface for a context object that exposes common functionality of an asset editor.
 */
class IAssetEditorContextInterface
{	
	GENERATED_BODY()

public:
	/**
	 * Get the selection set that the asset editor uses.
	 */
	virtual const UTypedElementSelectionSet* GetSelectionSet() const = 0;

	/**
	 * Get the selection set that the asset editor uses.
	 */
	virtual UTypedElementSelectionSet* GetMutableSelectionSet() = 0;

	/**
	 * Get the typed element common actions for the asset editor.
	 */
	virtual UTypedElementCommonActions* GetCommonActions() = 0;

	/**
	 * Gets the world on which the asset editor performs actions.
	 */
	virtual UWorld* GetEditingWorld() const = 0;

	/**
	 * Get the toolkit host associated with our asset editor.
	 */
	virtual const IToolkitHost* GetToolkitHost() const = 0;

	/**
	 * Get the toolkit host associated with our asset editor.
	 */
	virtual IToolkitHost* GetMutableToolkitHost() = 0;
};
