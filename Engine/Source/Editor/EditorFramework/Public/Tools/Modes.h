// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Math/NumericLimits.h"
#include "Misc/Attribute.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"

/** The shorthand identifier used for editor modes */
typedef FName FEditorModeID;

struct FEditorModeInfo
{
	/** Default constructor */
	EDITORFRAMEWORK_API FEditorModeInfo();

	/** Helper constructor */
	EDITORFRAMEWORK_API FEditorModeInfo(
		FEditorModeID InID,
		FText InName = FText(),
		FSlateIcon InIconBrush = FSlateIcon(),
		TAttribute<bool> InVisibility = false,
		int32 InPriorityOrder = MAX_int32
		);

	/** The mode ID */
	FEditorModeID ID;

	/** Name of the toolbar this mode uses and can be used by external systems to customize that mode toolbar */
	FName ToolbarCustomizationName;

	/** Name for the editor to display */
	FText Name;

	/** The mode icon */
	FSlateIcon IconBrush;

	/** The priority of this mode which will determine its default order and shift+X command assignment */
	int32 PriorityOrder;

	/** Whether or not the mode should be visible in the mode toolbar */
	EDITORFRAMEWORK_API bool IsVisible() const;

protected:
	TAttribute<bool> Visibility;
};

DECLARE_MULTICAST_DELEGATE(FRegisteredModesChangedEvent);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnModeRegistered, FEditorModeID);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnModeUnregistered, FEditorModeID);
