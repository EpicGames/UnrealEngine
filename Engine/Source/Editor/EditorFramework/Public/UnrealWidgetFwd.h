// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FWidget;

/** Coordinate system identifiers. */
enum ECoordSystem
{
	COORD_None = -1,
	COORD_World,
	COORD_Local,
	COORD_Parent,
	COORD_Explicit,
	COORD_Max,
};

namespace UE {
namespace Widget {
	enum EWidgetMode
	{
		WM_None = -1,
		WM_Translate,
		WM_TranslateRotateZ,
		WM_2D,
		WM_Rotate,
		WM_Scale,
		WM_Max,
	};
}
}
