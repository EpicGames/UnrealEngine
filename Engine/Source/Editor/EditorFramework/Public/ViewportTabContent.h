// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "EditorViewportLayout.h"
#include "Misc/Optional.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API EDITORFRAMEWORK_API

/**
 * Represents the content in a viewport tab in an editor.
 * Each SDockTab holding viewports in an editor contains and owns one of these.
 */

class FEditorViewportLayout;
class FUICommandList;
class IEditorViewportLayoutEntity;
class SDockTab;

class FViewportTabContent
{
public:
	virtual ~FViewportTabContent() = default;

	/** Returns whether the tab is currently shown */
	UE_API bool IsVisible() const;

	/** @return True if this viewport belongs to the tab given */
	UE_API bool BelongsToTab(TSharedRef<SDockTab> InParentTab) const;

	/**
	* Returns whether the named layout is currently selected
	*
	* @param ConfigurationName		The name of the layout (for the names in namespace LevelViewportConfigurationNames)
	* @return						True, if the named layout is currently active
	*/
	UE_API bool IsViewportConfigurationSet(const FName& ConfigurationName) const;

	virtual void SetViewportConfiguration(const FName& ConfigurationName) {}

	/**
	 * Maps the common commands into the given command list for this layout to the given viewport based on the config key.
	 * This call is only additive to the command list, and does not clear or erase already mapped actions.
	 */
	virtual void BindViewportLayoutCommands(FUICommandList& InOutCommandList, FName ViewportConfigKey) {}

	UE_API const TMap< FName, TSharedPtr< IEditorViewportLayoutEntity > >* GetViewports() const;

	using ViewportActionFunction = TFunction<void(FName Name, TSharedPtr<IEditorViewportLayoutEntity>)>;
	UE_API void PerformActionOnViewports(ViewportActionFunction& TFuncPtr);

	DECLARE_EVENT(FViewportTabContent, FViewportTabContentLayoutChangedEvent);
	virtual FViewportTabContentLayoutChangedEvent& OnViewportTabContentLayoutChanged() { return OnViewportTabContentLayoutChangedEvent; };

	DECLARE_EVENT_OneParam(FViewportTabContent, FViewportTabContentLayoutStartChangeEvent, bool);
	virtual FViewportTabContentLayoutStartChangeEvent& OnViewportTabContentLayoutStartChange() { return OnViewportTabContentLayoutStartChangeEvent; };

protected:
	FViewportTabContentLayoutChangedEvent OnViewportTabContentLayoutChangedEvent;
	FViewportTabContentLayoutStartChangeEvent OnViewportTabContentLayoutStartChangeEvent;

	TWeakPtr<SDockTab> ParentTab;

	FString LayoutString;

	/** Current layout */
	TSharedPtr<FEditorViewportLayout> ActiveViewportLayout;

	TOptional<FName> PreviouslyFocusedViewport;
};

#undef UE_API
