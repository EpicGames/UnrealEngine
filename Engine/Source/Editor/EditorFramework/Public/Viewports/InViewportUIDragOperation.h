// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Input/DragAndDrop.h"
#include "Layout/Visibility.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API EDITORFRAMEWORK_API

struct FPointerEvent;

DECLARE_DELEGATE_OneParam(FOnInViewportUIDropped, const FVector2D DropLocation)

class SDockingTabStack;

/** A Sample implementation of IDragDropOperation */
class FInViewportUIDragOperation : public FDragDropOperation
{
public:

	DRAG_DROP_OPERATOR_TYPE(FInViewportUIDragOperation, FDragDropOperation)

	/**
	* Invoked when the drag and drop operation has ended.
	*
	* @param bDropWasHandled   true when the drop was handled by some widget; false otherwise
	*/
	UE_API virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent) override;

	/**
	 * Called when the mouse was moved during a drag and drop operation
	 *
	 * @param DragDropEvent    The event that describes this drag drop operation.
	 */
	UE_API virtual void OnDragged(const FDragDropEvent& DragDropEvent) override;

	/**
	 * Create this Drag and Drop Content
	 *
	 * @param InUIToBeDragged	  The UI being dragged
	 * @param InDecoratorOffset   Where within the UI we grabbed, so we're not dragging by the upper left of the UI.
	 * @param OwnerAreaSize       Size of the DockArea at the time when we start dragging.
	 *
	 * @return a new FDockingDragOperation
	 */
	static UE_API TSharedRef<FInViewportUIDragOperation> New(const TSharedRef<class SWidget>& InUIToBeDragged, const FVector2D InDecoratorOffset, const FVector2D& OwnerAreaSize, FOnInViewportUIDropped& InDropDelegate);

	UE_API virtual ~FInViewportUIDragOperation();

	/** @return The offset into the tab where the user grabbed in Slate Units. */
	UE_API const FVector2f GetDecoratorOffsetFromCursor();

	UE_API void BroadcastDropEvent(const FVector2D InLocation);

protected:
	/** The constructor is protected, so that this class can only be instantiated as a shared pointer. */
	UE_API FInViewportUIDragOperation(const TSharedRef<class SWidget>& InUIToBeDragged, const FVector2D InDecoratorOffsetFromCursor, const FVector2D& OwnerAreaSize, FOnInViewportUIDropped& InDropDelegate);

	/** What is actually being dragged in this operation */
	TSharedPtr<class SWidget> UIBeingDragged;

	/** Where the user grabbed the UI measured in screen space from its top-left corner */
	FVector2f DecoratorOffsetFromCursor;

	/** Decorator widget where we add temp doc tabs to */
	TSharedPtr<SDockingTabStack> CursorDecoratorStackNode;

	/** What the size of the content was when it was last shown. The user drags splitters to set this size; it is legitimate state. */
	FVector2f LastContentSize;

	FOnInViewportUIDropped DropDelegate;
};

#undef UE_API
