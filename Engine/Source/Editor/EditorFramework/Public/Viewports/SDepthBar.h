// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SPanel.h"
#include "SlotBase.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/Children.h"
#include "Styling/DepthBarStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"

/**
 * A bar that allows for visualization and manipulation of a viewport's depth
 */
class UE_EXPERIMENTAL(5.7, "SDepthBar represents and controls view depth within viewports.") SDepthBar;
class SDepthBar : public SPanel
{
public:
	enum class EMode
	{
		Perspective,
		Orthographic
	};
	
	struct FDepthSpace
	{
		EDITORFRAMEWORK_API FDepthSpace(const FVector& InOrigin, const FVector& InForward, const FBox& Bounds);
		
		const FVector& GetPosition() const { return Origin; }
		const FVector& GetForward() const { return Forward; } 
		
		/** Gets the minimum value relative to originating direction */
		double GetAlignedMin() const { return AlignedMin; }
		/** Gets the maximum value relative to originating direction */
		double GetAlignedMax() const { return AlignedMax; }
		
		/** Get the point of the position on the line starting at the origin, facing along the forward. */
		EDITORFRAMEWORK_API double GetAlignedPosition(const FVector& Position) const;
		
		/** Get the relative value of the given axis-aligned position */
		EDITORFRAMEWORK_API double AlignedToRelativePosition(double AlignedPosition) const;
		
		/** Get the axis-aligned value of the given relative position */
		EDITORFRAMEWORK_API double RelativeToAlignedPosition(double RelativePosition) const;
		
		/** Get the bounds position of the given relative position */
		EDITORFRAMEWORK_API double RelativeToBoundsPosition(double RelativePosition) const;
		
		/** Get the relative value of the given bounds position */
		EDITORFRAMEWORK_API double BoundsToRelativePosition(double BoundsPosition) const;
		
	private:
		FVector Origin;
		FVector Forward;	
		
		double AlignedMin;
		double AlignedMax;
		
		bool bFlipped;
	};

	/**
	 * A slot that is placed along the depth bar, representing a range within the depth space
	 */
	class FDepthIndicatorSlot : public TBasicLayoutWidgetSlot<FDepthIndicatorSlot>
	{
	public:
	
		SLATE_SLOT_BEGIN_ARGS(FDepthIndicatorSlot, TBasicLayoutWidgetSlot<FDepthIndicatorSlot>)
			/** Determines where in the bar the slot's contents appear. */
			SLATE_ATTRIBUTE(FBoxSphereBounds, Bounds)
		SLATE_SLOT_END_ARGS()
		
		void SetBounds(TAttribute<FBoxSphereBounds> InBounds)
		{
			Bounds = InBounds;
		}
		
		void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
		{
			TBasicLayoutWidgetSlot<FDepthIndicatorSlot>::Construct(SlotOwner, MoveTemp(InArgs));
			Bounds = InArgs._Bounds;
		}
		
		const TAttribute<FBoxSphereBounds>& GetBounds() const { return Bounds; }
		
	protected:
		TAttribute<FBoxSphereBounds> Bounds;
	};

	static FDepthIndicatorSlot::FSlotArguments Slot()
	{
		return FDepthIndicatorSlot::FSlotArguments(MakeUnique<FDepthIndicatorSlot>());
	}
	
	DECLARE_DELEGATE_OneParam(FOnDepthChanged, const TOptional<double>&)
	
	SLATE_BEGIN_ARGS(SDepthBar)
	{}
		SLATE_ATTRIBUTE(EMode, Mode)
		SLATE_ATTRIBUTE(TOptional<FDepthSpace>, DepthSpace)
		
		SLATE_ATTRIBUTE(TOptional<double>, NearPlane)
		SLATE_ATTRIBUTE(TOptional<double>, FarPlane)
		
		SLATE_EVENT(FOnDepthChanged, OnNearPlaneChanged)
		SLATE_EVENT(FOnDepthChanged, OnFarPlaneChanged)
		
		SLATE_SLOT_ARGUMENT(SDepthBar::FDepthIndicatorSlot, Slots)
		
		PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		SLATE_STYLE_ARGUMENT(FDepthBarStyle, Style)
		PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	SLATE_END_ARGS()
	
	using FScopedWidgetDepthIndicatorSlotArguments = TPanelChildren<FDepthIndicatorSlot>::FScopedWidgetSlotArguments;
	FScopedWidgetDepthIndicatorSlotArguments AddDepthIndicatorSlot()
	{
		return InsertDepthIndicatorSlot(INDEX_NONE);
	}

	FScopedWidgetDepthIndicatorSlotArguments InsertDepthIndicatorSlot(int32 Index = INDEX_NONE)
	{
		return FScopedWidgetDepthIndicatorSlotArguments(MakeUnique<FDepthIndicatorSlot>(), this->DepthIndicatorChildren, Index);
	}
	
	EDITORFRAMEWORK_API FDepthIndicatorSlot& GetDepthIndicatorSlot(int32 Index);
	EDITORFRAMEWORK_API const FDepthIndicatorSlot& GetDepthIndicatorSlot(int32 Index) const;
	EDITORFRAMEWORK_API void ClearDepthIndicators();
	
	SDepthBar()
		: DepthIndicatorChildren(this)
		, DepthLabelChildren(this)
		, AllChildren(this)
	{
		AllChildren.AddChildren(DepthIndicatorChildren);
		AllChildren.AddChildren(DepthLabelChildren);
	}
	
	EDITORFRAMEWORK_API void Construct(const FArguments& InArgs);
	
	EDITORFRAMEWORK_API virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	EDITORFRAMEWORK_API virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;
	
	EDITORFRAMEWORK_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	EDITORFRAMEWORK_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	EDITORFRAMEWORK_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	EDITORFRAMEWORK_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	EDITORFRAMEWORK_API virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	
protected:
	EDITORFRAMEWORK_API virtual FChildren* GetChildren() override;
	EDITORFRAMEWORK_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	EDITORFRAMEWORK_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
	
private:
	
	struct FBarPositions
	{
		FDepthSpace DepthSpace;
	
		/** The Top of the bar (i.e. the far plane max) */
		double BarMin;
		/** The Bottom of the bar (i.e. the near plane min) */
		double BarMax;
		
		double BarFarPlane;
		double BarNearPlane;
		
		double WorldToBarPosition(const FVector& WorldPosition) const;
		double RelativeToBarPosition(double RelativePosition) const;
		double BarToAlignedPosition(double BarPosition) const;
	};

	DECLARE_DELEGATE_RetVal_OneParam(float, FGetBarPosition, const FBarPositions& BarPositions)

	// Depth label slots are displayed next to the bar
	struct FDepthLabelSlot : public TSlotBase<FDepthLabelSlot>
	{
		SLATE_SLOT_BEGIN_ARGS(FDepthLabelSlot, TSlotBase<FDepthLabelSlot>)
			/** Determines where in the bar the slot's contents appear. */
			SLATE_EVENT(FGetBarPosition, OnGetBarPosition)
			SLATE_ATTRIBUTE(EVerticalAlignment, VAlign);
		SLATE_SLOT_END_ARGS()
		
		void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
		{
			TSlotBase<FDepthLabelSlot>::Construct(SlotOwner, MoveTemp(InArgs));
			OnGetBarPosition = InArgs._OnGetBarPosition;
			VAlign = InArgs._VAlign;
		}
		
		float GetBarPosition(const FBarPositions& BarPositions) const
		{
			if (OnGetBarPosition.IsBound())
			{
				return OnGetBarPosition.Execute(BarPositions); 
			}
			return 0.0f;
		}
		
		EVerticalAlignment GetVerticalAlignment() const
		{
			if (VAlign.IsSet())
			{
				return VAlign.Get();
			}
			return VAlign_Center;
		}
	protected:
		FGetBarPosition OnGetBarPosition;
		TAttribute<EVerticalAlignment> VAlign = VAlign_Center;
	};
	
	using FScopedWidgetDepthLabelSlotArguments = TPanelChildren<FDepthLabelSlot>::FScopedWidgetSlotArguments;
	FScopedWidgetDepthLabelSlotArguments AddDepthLabelSlot()
	{
		return InsertDepthLabelSlot(INDEX_NONE);
	}

	FScopedWidgetDepthLabelSlotArguments InsertDepthLabelSlot(int32 Index = INDEX_NONE)
	{
		return FScopedWidgetDepthLabelSlotArguments(MakeUnique<FDepthLabelSlot>(), this->DepthLabelChildren, Index);
	}

	/** Gets the inset space where bar drawing occurs */
	FMargin GetBarInset() const;
	
	TOptional<FBarPositions> GetBarPositions(const FGeometry& InGeometry) const;
	
	EVisibility GetFarPlaneLabelVisibility() const;
	EVisibility GetNearPlaneLabelVisibility() const;
	FText GetFarPlaneText() const;
	FText GetNearPlaneText() const;
	
	TPanelChildren<FDepthIndicatorSlot> DepthIndicatorChildren;
	TPanelChildren<FDepthLabelSlot> DepthLabelChildren;
	FCombinedChildren AllChildren;
	
	enum class EDragTarget
	{
		None,
		
		Near_Button,
		Far_Button,
		
		Near_Handle,
		Far_Handle,
		Slice
	};
	EDragTarget HoverTarget = EDragTarget::None;
	EDragTarget DragTarget = EDragTarget::None;
	
	// Store drag position as doubles rather than floats to handle precision drags on very large worlds
	FVector2D StartDragPosition = FVector2D::Zero();
	FVector2D DragPosition = FVector2D::Zero();
	
	TAttribute<EMode> Mode;
	TAttribute<TOptional<FDepthSpace>> DepthSpace;
	TAttribute<TOptional<double>> NearPlane;
	TAttribute<TOptional<double>> FarPlane;
	FOnDepthChanged OnNearPlaneChanged;
	FOnDepthChanged OnFarPlaneChanged;
	
	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	const FDepthBarStyle* Style = nullptr;
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
};

